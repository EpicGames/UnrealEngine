// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Templates/Casts.h"
#include "EngineGlobals.h"
#include "PrimitiveSceneProxy.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/Engine.h"
#include "IHeadMountedDisplayModule.h"
#include "IHeadMountedDisplayVulkanExtensions.h"
#include "IHeadMountedDisplay.h"
#include "PrimitiveSceneInfo.h"


class FHeadMountedDisplayModule : public IHeadMountedDisplayModule
{
	virtual TSharedPtr< class IXRTrackingSystem, ESPMode::ThreadSafe > CreateTrackingSystem()
	{
		TSharedPtr<IXRTrackingSystem, ESPMode::ThreadSafe> DummyVal = nullptr;
		return DummyVal;
	}

	FString GetModuleKeyName() const
	{
		return FString("Default");
	}
};

IMPLEMENT_MODULE( FHeadMountedDisplayModule, HeadMountedDisplay );

IHeadMountedDisplay::IHeadMountedDisplay()
{
}

bool IHeadMountedDisplay::DoesAppUseVRFocus() const
{
	return FApp::UseVRFocus();
}

bool IHeadMountedDisplay::DoesAppHaveVRFocus() const
{
	return FApp::HasVRFocus();
}


