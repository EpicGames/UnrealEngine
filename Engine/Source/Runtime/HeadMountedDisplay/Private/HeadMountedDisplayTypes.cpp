// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadMountedDisplayTypes.h"
#include "RendererInterface.h"
#include "CommonRenderResources.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HeadMountedDisplayTypes)

DEFINE_LOG_CATEGORY(LogHMD);
DEFINE_LOG_CATEGORY(LogLoadingSplash);

FHMDViewMesh::FHMDViewMesh() :
	NumVertices(0),
	NumIndices(0),
	NumTriangles(0)
{}

FHMDViewMesh::~FHMDViewMesh()
{
}

void FHMDViewMesh::BuildMesh(const FVector2D Positions[], uint32 VertexCount, EHMDMeshType MeshType)
{
	check(VertexCount > 2 && VertexCount % 3 == 0);
	FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

	NumVertices = VertexCount;
	NumTriangles = NumVertices / 3;
	NumIndices = NumVertices;

	const FRHIBufferCreateDesc VertexCreateDesc =
		FRHIBufferCreateDesc::CreateVertex<FFilterVertex>(TEXT("FHMDViewMesh"), NumVertices)
		.AddUsage(EBufferUsageFlags::Static)
		.SetInitActionInitializer()
		.DetermineInitialState();

	const FRHIBufferCreateDesc IndexCreateDesc =
		FRHIBufferCreateDesc::CreateIndex<uint16>(TEXT("FHMDViewMesh"), NumIndices)
		.AddUsage(EBufferUsageFlags::Static)
		.SetInitActionInitializer()
		.DetermineInitialState();

	TRHIBufferInitializer<FFilterVertex> pVertices = RHICmdList.CreateBufferInitializer(VertexCreateDesc);
	TRHIBufferInitializer<uint16> pIndices = RHICmdList.CreateBufferInitializer(IndexCreateDesc);

	for (uint32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		const FVector2D& Position = Positions[VertexIndex];
		FFilterVertex& Vertex = pVertices[VertexIndex];

		if (MeshType == MT_HiddenArea)
		{
			// Remap from to NDC space [0 1] -> [-1 1]
			Vertex.Position.X = ((float)Position.X * 2.0f) - 1.0f;
			Vertex.Position.Y = ((float)Position.Y * 2.0f) - 1.0f;
			Vertex.Position.Z = 1.0f;
			Vertex.Position.W = 1.0f;

			// Not used for hidden area
			Vertex.UV.X = 0.0f;
			Vertex.UV.Y = 0.0f;
		}
		else
		{
			// Remap the viewport origin from the bottom left to the top left
			Vertex.Position.X = (float)Position.X;
			Vertex.Position.Y = 1.0f - (float)Position.Y;
			Vertex.Position.Z = 0.0f;
			Vertex.Position.W = 1.0f;

			Vertex.UV.X = (float)Position.X;
			Vertex.UV.Y = 1.0f - (float)Position.Y;
		}

		pIndices[VertexIndex] = static_cast<uint16>(VertexIndex);
	}

	VertexBufferRHI = pVertices.Finalize();
	IndexBufferRHI = pIndices.Finalize();
}
