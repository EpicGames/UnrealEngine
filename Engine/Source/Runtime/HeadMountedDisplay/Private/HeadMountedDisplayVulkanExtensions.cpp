// Copyright Epic Games, Inc. All Rights Reserved.

#include "IHeadMountedDisplayVulkanExtensions.h"
#include "GeneralProjectSettings.h"
#include "Misc/CommandLine.h"
#include "StereoRendering.h"

bool IHeadMountedDisplayVulkanExtensions::ShouldDisableVulkanVSync() const
{
	return IStereoRendering::IsStartInVR();
}
