// Copyright Epic Games, Inc. All Rights Reserved.

#include "IHandTracker.h"

// This file exists so that the static methods of IHandTracker are compiled.