// Copyright Epic Games, Inc. All Rights Reserved.

#include "IIdentifiableXRDevice.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IIdentifiableXRDevice)

/* FXRDeviceId
 *****************************************************************************/

FXRDeviceId::FXRDeviceId()
	: SystemName(NAME_None)
	, DeviceId(INDEX_NONE)
{}

FXRDeviceId::FXRDeviceId(IIdentifiableXRDevice* InDeviceId)
	: SystemName(InDeviceId->GetSystemName())
	, DeviceId(InDeviceId->GetSystemDeviceId())
{}

FXRDeviceId::FXRDeviceId(IXRSystemIdentifier* OwningSystem, const int32 InDeviceId)
	: SystemName(OwningSystem->GetSystemName())
	, DeviceId(InDeviceId)
{}

bool FXRDeviceId::IsOwnedBy(IXRSystemIdentifier* XRSystem) const
{
	return XRSystem && XRSystem->GetSystemName() == SystemName;
}

void FXRDeviceId::Clear()
{
	SystemName = NAME_None;
	DeviceId = INDEX_NONE;
}

bool FXRDeviceId::operator==(const FXRDeviceId& Rhs) const
{
	return SystemName == Rhs.SystemName && DeviceId == Rhs.DeviceId;
}

bool FXRDeviceId::operator==(const IIdentifiableXRDevice* Rhs) const
{
	return Rhs && SystemName == Rhs->GetSystemName() && DeviceId == Rhs->GetSystemDeviceId();
}

