// Copyright Epic Games, Inc. All Rights Reserved.

#include "IMotionController.h"
#include "InputCoreTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IMotionController)

FName IMotionController::LeftHandSourceId(TEXT("Left"));
FName IMotionController::RightHandSourceId(TEXT("Right"));
FName IMotionController::HMDSourceId(TEXT("HMD"));
FName IMotionController::HeadSourceId(TEXT("Head"));

bool IMotionController::GetHandEnumForSourceName(const FName Source, EControllerHand& OutHand)
{
	static TMap<FName, EControllerHand> MotionSourceToEControllerHandMap;
	if (MotionSourceToEControllerHandMap.Num() == 0)
	{
		MotionSourceToEControllerHandMap.Reserve(40);

		// Motion source names that map to legacy EControllerHand values
		MotionSourceToEControllerHandMap.Add(IMotionController::LeftHandSourceId, EControllerHand::Left);
		MotionSourceToEControllerHandMap.Add(IMotionController::RightHandSourceId, EControllerHand::Right);
		MotionSourceToEControllerHandMap.Add(TEXT("AnyHand"), EControllerHand::AnyHand);
		MotionSourceToEControllerHandMap.Add(TEXT("Pad"), EControllerHand::Pad);
		MotionSourceToEControllerHandMap.Add(TEXT("ExternalCamera"), EControllerHand::ExternalCamera);
		MotionSourceToEControllerHandMap.Add(TEXT("Gun"), EControllerHand::Gun);
		MotionSourceToEControllerHandMap.Add(TEXT("HMD"), EControllerHand::HMD);
		MotionSourceToEControllerHandMap.Add(TEXT("Chest"), EControllerHand::Chest);
		MotionSourceToEControllerHandMap.Add(TEXT("LeftShoulder"), EControllerHand::LeftShoulder);
		MotionSourceToEControllerHandMap.Add(TEXT("RightShoulder"), EControllerHand::RightShoulder);
		MotionSourceToEControllerHandMap.Add(TEXT("LeftElbow"), EControllerHand::LeftElbow);
		MotionSourceToEControllerHandMap.Add(TEXT("RightElbow"), EControllerHand::RightElbow);
		MotionSourceToEControllerHandMap.Add(TEXT("Waist"), EControllerHand::Waist);
		MotionSourceToEControllerHandMap.Add(TEXT("LeftKnee"), EControllerHand::LeftKnee);
		MotionSourceToEControllerHandMap.Add(TEXT("RightKnee"), EControllerHand::RightKnee);
		MotionSourceToEControllerHandMap.Add(TEXT("LeftFoot"), EControllerHand::LeftFoot);
		MotionSourceToEControllerHandMap.Add(TEXT("RightFoot"), EControllerHand::RightFoot);
		MotionSourceToEControllerHandMap.Add(TEXT("Special"), EControllerHand::Special);
		// EControllerHand enum names mapped to EControllerHand enum values
		MotionSourceToEControllerHandMap.Add(TEXT("EControllerHand::Left"), EControllerHand::Left);
		MotionSourceToEControllerHandMap.Add(TEXT("EControllerHand::Right"), EControllerHand::Right);
		MotionSourceToEControllerHandMap.Add(TEXT("EControllerHand::AnyHand"), EControllerHand::AnyHand);
		MotionSourceToEControllerHandMap.Add(TEXT("EControllerHand::Pad"), EControllerHand::Pad);
		MotionSourceToEControllerHandMap.Add(TEXT("EControllerHand::ExternalCamera"), EControllerHand::ExternalCamera);
		MotionSourceToEControllerHandMap.Add(TEXT("EControllerHand::Gun"), EControllerHand::Gun);
		MotionSourceToEControllerHandMap.Add(TEXT("EControllerHand::HMD"), EControllerHand::HMD);
		MotionSourceToEControllerHandMap.Add(TEXT("EControllerHand::Chest"), EControllerHand::Chest);
		MotionSourceToEControllerHandMap.Add(TEXT("EControllerHand::LeftShoulder"), EControllerHand::LeftShoulder);
		MotionSourceToEControllerHandMap.Add(TEXT("EControllerHand::RightShoulder"), EControllerHand::RightShoulder);
		MotionSourceToEControllerHandMap.Add(TEXT("EControllerHand::LeftElbow"), EControllerHand::LeftElbow);
		MotionSourceToEControllerHandMap.Add(TEXT("EControllerHand::RightElbow"), EControllerHand::RightElbow);
		MotionSourceToEControllerHandMap.Add(TEXT("EControllerHand::Waist"), EControllerHand::Waist);
		MotionSourceToEControllerHandMap.Add(TEXT("EControllerHand::LeftKnee"), EControllerHand::LeftKnee);
		MotionSourceToEControllerHandMap.Add(TEXT("EControllerHand::RightKnee"), EControllerHand::RightKnee);
		MotionSourceToEControllerHandMap.Add(TEXT("EControllerHand::LeftFoot"), EControllerHand::LeftFoot);
		MotionSourceToEControllerHandMap.Add(TEXT("EControllerHand::RightFoot"), EControllerHand::RightFoot);
		MotionSourceToEControllerHandMap.Add(TEXT("EControllerHand::Special"), EControllerHand::Special);
		// Newer source names that can usefully map to legacy EControllerHand values
		MotionSourceToEControllerHandMap.Add(TEXT("LeftGrip"), EControllerHand::Left);
		MotionSourceToEControllerHandMap.Add(TEXT("RightGrip"), EControllerHand::Right);
		MotionSourceToEControllerHandMap.Add(TEXT("LeftAim"), EControllerHand::Left);
		MotionSourceToEControllerHandMap.Add(TEXT("RightAim"), EControllerHand::Right);
		MotionSourceToEControllerHandMap.Add(IMotionController::HeadSourceId, EControllerHand::HMD);
	}

	EControllerHand* FoundEnum = MotionSourceToEControllerHandMap.Find(Source);
	if (FoundEnum != nullptr)
	{
		OutHand = *FoundEnum;
		return true;
	}
	else
	{
		return false;
	}
}
