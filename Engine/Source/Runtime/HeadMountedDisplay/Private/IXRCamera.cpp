// Copyright Epic Games, Inc. All Rights Reserved.

#include "IXRCamera.h"
