// Copyright Epic Games, Inc. All Rights Reserved.

#include "IXRTrackingSystem.h"
#include "IHeadMountedDisplay.h"
#include "EngineGlobals.h"
#include "RenderGraphBuilder.h"
#include "Engine/Engine.h"
#include "Kismet/GameplayStatics.h"
#include "Misc/App.h"

const int32 IXRTrackingSystem::HMDDeviceId;

void IXRTrackingSystem::OnBeginRendering_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& ViewFamily)
{
}

void IXRTrackingSystem::OnLateUpdateApplied_RenderThread(FRDGBuilder& GraphBuilder, const FTransform& NewRelativeTransform)
{
}

void IXRTrackingSystem::GetHMDData(UObject* WorldContext, FXRHMDData& HMDData)
{
	HMDData.DeviceName = GetHMDDevice() ? GetHMDDevice()->GetHMDName() : GetSystemName();
	HMDData.ApplicationInstanceID = FApp::GetInstanceId();

	bool bIsTracking = IsTracking(IXRTrackingSystem::HMDDeviceId);
	HMDData.TrackingStatus = bIsTracking ? ETrackingStatus::Tracked : ETrackingStatus::NotTracked;

	HMDData.bValid = GetCurrentPose(IXRTrackingSystem::HMDDeviceId, HMDData.Rotation, HMDData.Position);
}

bool IXRTrackingSystem::IsHeadTrackingAllowedForWorld(UWorld& World) const
{
#if WITH_EDITOR
	// For VR PIE only the primary instance uses the headset.

	if (!IsHeadTrackingAllowed())
	{
		return false;
	}

	if (World.WorldType != EWorldType::PIE)
	{
		return true;
	}

	FWorldContext* const WorldContext = GEngine->GetWorldContextFromWorld(&World);
	return WorldContext && WorldContext->bIsPrimaryPIEInstance;
#else
	return IsHeadTrackingAllowed();
#endif
}
