// Copyright Epic Games, Inc. All Rights Reserved.

#include "LateUpdateManager.h"
#include "PrimitiveSceneProxy.h"
#include "Components/PrimitiveComponent.h"
#include "PrimitiveSceneInfo.h"
#include "HeadMountedDisplayTypes.h"
#include "SceneInterface.h"

static TAutoConsoleVariable<bool> CVarXRLateUpdateMangerDisable(
	TEXT("xr.LateUpdateManager.Disable"),
	false,
	TEXT("Disable the LateUpdateManager preventing child components from receiving late updates.\n"),
	ECVF_Default);

void FLateUpdateManager::Setup(const FTransform& ParentToWorld, USceneComponent* Component, bool bSkipLateUpdate)
{
	check(IsInGameThread());
	bSkipLateUpdate = bSkipLateUpdate || CVarXRLateUpdateMangerDisable.GetValueOnGameThread();

	PipelinedUpdateStatesGame.Primitives.Reset();
	PipelinedUpdateStatesGame.ParentToWorld = ParentToWorld;
	GatherLateUpdatePrimitives(Component);
	PipelinedUpdateStatesGame.bSkip = bSkipLateUpdate;

	ENQUEUE_RENDER_COMMAND(UpdateLateUpdateStatesRendering)(
		[UpdateStatesGame = PipelinedUpdateStatesGame, this](FRHICommandListImmediate& RHICmdList) mutable
		{
			PipelinedUpdateStatesRendering = MoveTemp(UpdateStatesGame);
		});
}

void FLateUpdateManager::Apply_RenderThread(FSceneInterface* Scene, const FTransform& OldRelativeTransform, const FTransform& NewRelativeTransform)
{
	FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

	if (!PipelinedUpdateStatesRendering.Primitives.Num() || PipelinedUpdateStatesRendering.bSkip)
	{
		return;
	}

	const FTransform OldCameraTransform = OldRelativeTransform * PipelinedUpdateStatesRendering.ParentToWorld;
	const FTransform NewCameraTransform = NewRelativeTransform * PipelinedUpdateStatesRendering.ParentToWorld;
	const FMatrix LateUpdateTransform = (OldCameraTransform.Inverse() * NewCameraTransform).ToMatrixWithScale();

	bool bIndicesHaveChanged = false;

	// Apply delta to the cached scene proxies
	// Also check whether any primitive indices have changed, in case the scene has been modified in the meantime.
	for (auto& PrimitivePair : PipelinedUpdateStatesRendering.Primitives)
	{
		FPrimitiveSceneInfo* RetrievedSceneInfo = Scene->GetPrimitiveSceneInfo(PrimitivePair.Value);
		FPrimitiveSceneInfo* CachedSceneInfo = PrimitivePair.Key;

		// If the retrieved scene info is different than our cached scene info then the scene has changed in the meantime
		// and we need to search through the entire scene to make sure it still exists.
		if (CachedSceneInfo != RetrievedSceneInfo)
		{
			bIndicesHaveChanged = true;
			break; // No need to continue here, as we are going to brute force the scene primitives below anyway.
		}
		else if (CachedSceneInfo->Proxy)
		{
			CachedSceneInfo->Proxy->ApplyLateUpdateTransform(RHICmdList, LateUpdateTransform);
			PrimitivePair.Value = -1; // Set the cached index to -1 to indicate that this primitive was already processed
		}
	}

	// Indices have changed, so we need to scan the entire scene for primitives that might still exist
	if (bIndicesHaveChanged)
	{
		int32 Index = 0;
		FPrimitiveSceneInfo* RetrievedSceneInfo;
		RetrievedSceneInfo = Scene->GetPrimitiveSceneInfo(Index++);
		while(RetrievedSceneInfo)
		{
			if (RetrievedSceneInfo->Proxy && PipelinedUpdateStatesRendering.Primitives.Contains(RetrievedSceneInfo) && PipelinedUpdateStatesRendering.Primitives[RetrievedSceneInfo] >= 0)
			{
				RetrievedSceneInfo->Proxy->ApplyLateUpdateTransform(RHICmdList, LateUpdateTransform);
			}
			RetrievedSceneInfo = Scene->GetPrimitiveSceneInfo(Index++);
		}
	}
}

void FLateUpdateManager::CacheSceneInfo(USceneComponent* Component)
{
	ensureMsgf(!Component->IsUsingAbsoluteLocation() && !Component->IsUsingAbsoluteRotation(), TEXT("SceneComponents that use absolute location or rotation are not supported by the LateUpdateManager"));
	// If a scene proxy is present, cache it
	UPrimitiveComponent* PrimitiveComponent = dynamic_cast<UPrimitiveComponent*>(Component);
	if (PrimitiveComponent && PrimitiveComponent->SceneProxy)
	{
		FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveComponent->SceneProxy->GetPrimitiveSceneInfo();
		if (PrimitiveSceneInfo && PrimitiveSceneInfo->IsIndexValid())
		{
			PipelinedUpdateStatesGame.Primitives.Emplace(PrimitiveSceneInfo, PrimitiveSceneInfo->GetIndex());
		}
	}
}

void FLateUpdateManager::GatherLateUpdatePrimitives(USceneComponent* ParentComponent)
{
	CacheSceneInfo(ParentComponent);

	TArray<USceneComponent*> Components;
	ParentComponent->GetChildrenComponents(true, Components);
	for(USceneComponent* Component : Components)
	{
		if (Component != nullptr)
		{
			CacheSceneInfo(Component);
		}
	}
}
