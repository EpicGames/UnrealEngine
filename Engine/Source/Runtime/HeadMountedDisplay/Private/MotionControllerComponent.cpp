// Copyright Epic Games, Inc. All Rights Reserved.
//
#include "MotionControllerComponent.h"
#include "PrimitiveSceneProxy.h"
#include "Misc/ScopeLock.h"
#include "Features/IModularFeatures.h"
#include "GameFramework/WorldSettings.h"
#include "MotionDelayBuffer.h"
#include "SceneView.h"
#include "UObject/VRObjectVersion.h"
#include "IXRTrackingSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MotionControllerComponent)

DEFINE_LOG_CATEGORY_STATIC(LogMotionControllerComponent, Log, All);

UMotionControllerComponent::FActivateVisualizationComponent UMotionControllerComponent::OnActivateVisualizationComponent;


namespace {
	/** This is to prevent destruction of motion controller components while they are
		in the middle of being accessed by the render thread */
	FCriticalSection CritSect;

	/** Console variable for specifying whether motion controller late update is used */
	TAutoConsoleVariable<int32> CVarEnableMotionControllerLateUpdate(
		TEXT("vr.EnableMotionControllerLateUpdate"),
		1,
		TEXT("This command allows you to specify whether the motion controller late update is applied.\n")
		TEXT(" 0: don't use late update\n")
		TEXT(" 1: use late update (default)"),
		ECVF_Cheat);
} // anonymous namespace

namespace LegacyMotionSources
{
	static bool GetSourceNameForHand(EControllerHand InHand, FName& OutSourceName)
	{
		UEnum* HandEnum = StaticEnum<EControllerHand>();
		if (HandEnum)
		{
			FString ValueName = HandEnum->GetNameStringByValue((int64)InHand);
			if (!ValueName.IsEmpty())
			{
				OutSourceName = *ValueName;
				return true;
			}			
		}
		return false;
	}
}

//=============================================================================
UMotionControllerComponent::UMotionControllerComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, RenderThreadComponentScale(1.0f,1.0f,1.0f)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	PrimaryComponentTick.TickGroup = TG_PrePhysics;
	PrimaryComponentTick.bTickEvenWhenPaused = true;

	PlayerIndex = 0;
	MotionSource = IMotionController::LeftHandSourceId;
	bDisableLowLatencyUpdate = false;
	bHasAuthority = false;
	bAutoActivate = true;

	// ensure InitializeComponent() gets called
	bWantsInitializeComponent = true;
}

//=============================================================================
void UMotionControllerComponent::BeginDestroy()
{
	Super::BeginDestroy();
	if (ViewExtension.IsValid())
	{
		{
			// This component could be getting accessed from the render thread so it needs to wait
			// before clearing MotionControllerComponent and allowing the destructor to continue
			FScopeLock ScopeLock(&CritSect);
			ViewExtension->MotionControllerComponent = NULL;
		}

		ViewExtension.Reset();
	}
}

//=============================================================================
FName UMotionControllerComponent::GetTrackingMotionSource() 
{
	return MotionSource;
}

//=============================================================================
void UMotionControllerComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (IsActive())
	{
		FVector Position = GetRelativeTransform().GetTranslation();
		FRotator Orientation = GetRelativeTransform().GetRotation().Rotator();
		float WorldToMeters = GetWorld() ? GetWorld()->GetWorldSettings()->WorldToMeters : 100.0f;
		const bool bNewTrackedState = PollControllerState_GameThread(Position, Orientation, bProvidedLinearVelocity, LinearVelocity, bProvidedAngularVelocity, AngularVelocityAsAxisAndLength, bProvidedLinearAcceleration, LinearAcceleration, WorldToMeters);
		if (bNewTrackedState)
		{
			// Only update the location and rotation if we are tracking because we want the controller to stay in place rather than pop to 0,0,0.  
			// Note we do update the velocity and acceleration values even if untracked because we won't see any change in the position this frame.
			// This means that for brief tracking dropouts position and orientation should behave somewhat gracefully even without interpolation, but velocity/acceleration will show snaps to zero.
			SetRelativeLocationAndRotation(Position, Orientation);
		}

		// if controller tracking just kicked in or we haven't started rendering in the (possibly present) 
		// visualization component.
		if (!bTracked && bNewTrackedState)
		{
			OnActivateVisualizationComponent.Broadcast(true);
		}

		bTracked = bNewTrackedState;

		if (!ViewExtension.IsValid() && GEngine)
		{
			ViewExtension = FSceneViewExtensions::NewExtension<FViewExtension>(this);
		}
	}
}

//=============================================================================
void UMotionControllerComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	Super::CreateRenderState_Concurrent(Context);
	RenderThreadRelativeTransform = GetRelativeTransform();
	RenderThreadComponentScale = GetComponentScale();
}

//=============================================================================
void UMotionControllerComponent::SendRenderTransform_Concurrent()
{
	struct FPrimitiveUpdateRenderThreadRelativeTransformParams
	{
		FTransform RenderThreadRelativeTransform;
		FVector RenderThreadComponentScale;
	};

	FPrimitiveUpdateRenderThreadRelativeTransformParams UpdateParams;
	UpdateParams.RenderThreadRelativeTransform = GetRelativeTransform();
	UpdateParams.RenderThreadComponentScale = GetComponentScale();

	ENQUEUE_RENDER_COMMAND(UpdateRTRelativeTransformCommand)(
		[UpdateParams, this](FRHICommandListImmediate& RHICmdList)
	{
		RenderThreadRelativeTransform = UpdateParams.RenderThreadRelativeTransform;
		RenderThreadComponentScale = UpdateParams.RenderThreadComponentScale;
	});

	Super::SendRenderTransform_Concurrent();
}

//=============================================================================
void UMotionControllerComponent::SetTrackingSource(const EControllerHand NewSource)
{
	if (LegacyMotionSources::GetSourceNameForHand(NewSource, MotionSource))
	{
		UWorld* MyWorld = GetWorld();
		if (MyWorld && MyWorld->IsGameWorld() && HasBeenInitialized())
		{
			FMotionDelayService::RegisterDelayTarget(this, PlayerIndex, MotionSource);
		}
	}
}

//=============================================================================
EControllerHand UMotionControllerComponent::GetTrackingSource() const
{
	EControllerHand Hand = EControllerHand::Left;
	IMotionController::GetHandEnumForSourceName(MotionSource, Hand);
	return Hand;
}

//=============================================================================
void UMotionControllerComponent::SetTrackingMotionSource(const FName NewSource)
{
	MotionSource = NewSource;

	UWorld* MyWorld = GetWorld();
	if (MyWorld && MyWorld->IsGameWorld() && HasBeenInitialized())
	{
		FMotionDelayService::RegisterDelayTarget(this, PlayerIndex, NewSource);
	}
}

//=============================================================================
void UMotionControllerComponent::SetAssociatedPlayerIndex(const int32 NewPlayer)
{
	PlayerIndex = NewPlayer;

	UWorld* MyWorld = GetWorld();
	if (MyWorld && MyWorld->IsGameWorld() && HasBeenInitialized())
	{
		FMotionDelayService::RegisterDelayTarget(this, NewPlayer, MotionSource);
	}
}

void UMotionControllerComponent::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FVRObjectVersion::GUID);

	Super::Serialize(Ar);
}

//=============================================================================
void UMotionControllerComponent::InitializeComponent()
{
	Super::InitializeComponent();

	UWorld* MyWorld = GetWorld();
	if (MyWorld && MyWorld->IsGameWorld())
	{
		FMotionDelayService::RegisterDelayTarget(this, PlayerIndex, MotionSource);
	}
}

namespace UEMotionController {
	// A scoped lock that must be explicitly locked and will unlock upon destruction if locked.
	// Convenient if you only sometimes want to lock and the scopes are complicated.
	class FScopeLockOptional
	{
	public:
		FScopeLockOptional()
		{
		}

		void Lock(FCriticalSection* InSynchObject)
		{
			SynchObject = InSynchObject;
			SynchObject->Lock();
		}

		/** Destructor that performs a release on the synchronization object. */
		~FScopeLockOptional()
		{
			Unlock();
		}

		void Unlock()
		{
			if (SynchObject)
			{
				SynchObject->Unlock();
				SynchObject = nullptr;
			}
		}

	private:
		/** Copy constructor( hidden on purpose). */
		FScopeLockOptional(const FScopeLockOptional& InScopeLock);

		/** Assignment operator (hidden on purpose). */
		FScopeLockOptional& operator=(FScopeLockOptional& InScopeLock)
		{
			return *this;
		}

	private:

		// Holds the synchronization object to aggregate and scope manage.
		FCriticalSection* SynchObject = nullptr;
	};
}

//=============================================================================
bool UMotionControllerComponent::PollControllerState_GameThread(FVector& Position, FRotator& Orientation, bool& OutbProvidedLinearVelocity, FVector& OutLinearVelocity, bool& OutbProvidedAngularVelocity, FVector& OutAngularVelocityAsAxisAndLength, bool& OutbProvidedLinearAcceleration, FVector& OutLinearAcceleration, float WorldToMetersScale)
{
	check(IsInGameThread());

	// Cache state from the game thread for use on the render thread
	const AActor* MyOwner = GetOwner();
	bHasAuthority = MyOwner->HasLocalNetOwner();

	if(bHasAuthority)
	{
		{
			FScopeLock Lock(&PolledMotionControllerMutex);
			PolledMotionController_GameThread = nullptr;
			bPolledHMD_GameThread = false;
		}

		if (MotionSource == IMotionController::HMDSourceId || MotionSource == IMotionController::HeadSourceId)
		{
			IXRTrackingSystem* TrackingSys = GEngine->XRSystem.Get();
			if (TrackingSys)
			{
				FQuat OrientationQuat;
				if (TrackingSys->GetCurrentPose(IXRTrackingSystem::HMDDeviceId, OrientationQuat, Position))
				{
					Orientation = OrientationQuat.Rotator();
					{
						FScopeLock Lock(&PolledMotionControllerMutex);
						bPolledHMD_GameThread = true;  // We only want a render thread update from the hmd if we polled it on the game thread.
					}
					return true;
				}
			}
		}
		else
		{
			TArray<IMotionController*> MotionControllers;
			MotionControllers = IModularFeatures::Get().GetModularFeatureImplementations<IMotionController>(IMotionController::GetModularFeatureName());
			for (auto MotionController : MotionControllers)
			{
				if (MotionController == nullptr)
				{
					continue;
				}

				CurrentTrackingStatus = MotionController->GetControllerTrackingStatus(PlayerIndex, MotionSource);
				if (MotionController->GetControllerOrientationAndPosition(PlayerIndex, MotionSource, Orientation, Position, OutbProvidedLinearVelocity, OutLinearVelocity, OutbProvidedAngularVelocity, OutAngularVelocityAsAxisAndLength, OutbProvidedLinearAcceleration, OutLinearAcceleration, WorldToMetersScale))
				{
					InUseMotionController = MotionController;
					OnMotionControllerUpdated();
					InUseMotionController = nullptr;

					{
						FScopeLock Lock(&PolledMotionControllerMutex);
						PolledMotionController_GameThread = MotionController;  // We only want a render thread update from the motion controller we polled on the game thread.
					}
					return true;
				}
			}
		}
	}
	return false;
}

bool UMotionControllerComponent::PollControllerState_RenderThread(FVector& Position, FRotator& Orientation, float WorldToMetersScale)
{
	check(IsInRenderingThread());

	if (PolledMotionController_RenderThread)
	{
		CurrentTrackingStatus = PolledMotionController_RenderThread->GetControllerTrackingStatus(PlayerIndex, MotionSource);
		if (PolledMotionController_RenderThread->GetControllerOrientationAndPosition(PlayerIndex, MotionSource, Orientation, Position, WorldToMetersScale))
		{
			return true;
		}
	}

	if (bPolledHMD_RenderThread)
	{
		IXRTrackingSystem* TrackingSys = GEngine->XRSystem.Get();
		if (TrackingSys)
		{
			FQuat OrientationQuat;
			if (TrackingSys->GetCurrentPose(IXRTrackingSystem::HMDDeviceId, OrientationQuat, Position))
			{
				Orientation = OrientationQuat.Rotator();
				return true;
			}
		}
	}

	return false;
}

//=============================================================================
void UMotionControllerComponent::OnModularFeatureUnregistered(const FName& Type, class IModularFeature* ModularFeature)
{
	FScopeLock Lock(&PolledMotionControllerMutex);

	if (ModularFeature == PolledMotionController_GameThread)
	{
		PolledMotionController_GameThread = nullptr;
	}
	if (ModularFeature == PolledMotionController_RenderThread)
	{
		PolledMotionController_RenderThread = nullptr;
	}
}

//=============================================================================
UMotionControllerComponent::FViewExtension::FViewExtension(const FAutoRegister& AutoRegister, UMotionControllerComponent* InMotionControllerComponent)
	: FSceneViewExtensionBase(AutoRegister)
	, MotionControllerComponent(InMotionControllerComponent)
{}

//=============================================================================
void UMotionControllerComponent::FViewExtension::BeginRenderViewFamily(FSceneViewFamily& InViewFamily)
{
	if (!MotionControllerComponent)
	{
		return;
	}

	// Set up the late update state for the controller component
	LateUpdate.Setup(MotionControllerComponent->CalcNewComponentToWorld(FTransform()), MotionControllerComponent, false);
}

//=============================================================================
void UMotionControllerComponent::FViewExtension::PreRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily)
{
	if (!MotionControllerComponent)
	{
		return;
	}

	FTransform OldTransform;
	FTransform NewTransform;
	{
		FScopeLock ScopeLock(&CritSect);
		if (!MotionControllerComponent)
		{
			return;
		}

		{
			FScopeLock Lock(&MotionControllerComponent->PolledMotionControllerMutex);
			MotionControllerComponent->PolledMotionController_RenderThread = MotionControllerComponent->PolledMotionController_GameThread;
			MotionControllerComponent->bPolledHMD_RenderThread = MotionControllerComponent->bPolledHMD_GameThread;
		}

		// Find a view that is associated with this player.
		float WorldToMetersScale = -1.0f;
		for (const FSceneView* SceneView : InViewFamily.Views)
		{
			if (SceneView && SceneView->PlayerIndex == MotionControllerComponent->PlayerIndex)
			{
				WorldToMetersScale = SceneView->WorldToMetersScale;
				break;
			}
		}
		// If there are no views associated with this player use view 0.
		if (WorldToMetersScale < 0.0f)
		{
			check(InViewFamily.Views.Num() > 0);
			WorldToMetersScale = InViewFamily.Views[0]->WorldToMetersScale;
		}

		// Poll state for the most recent controller transform
		FVector Position = MotionControllerComponent->RenderThreadRelativeTransform.GetTranslation();
		FRotator Orientation = MotionControllerComponent->RenderThreadRelativeTransform.GetRotation().Rotator();
		if (!MotionControllerComponent->PollControllerState_RenderThread(Position, Orientation, WorldToMetersScale))
		{
			return;
		}

		OldTransform = MotionControllerComponent->RenderThreadRelativeTransform;
		NewTransform = FTransform(Orientation, Position, MotionControllerComponent->RenderThreadComponentScale);
	} // Release the lock on the MotionControllerComponent

	// Tell the late update manager to apply the offset to the scene components
	LateUpdate.Apply_RenderThread(InViewFamily.Scene, OldTransform, NewTransform);
}

//=============================================================================
bool UMotionControllerComponent::FViewExtension::IsActiveThisFrame_Internal(const FSceneViewExtensionContext&) const
{
	check(IsInGameThread());
	return MotionControllerComponent && !MotionControllerComponent->bDisableLowLatencyUpdate && CVarEnableMotionControllerLateUpdate.GetValueOnGameThread();
}

//=============================================================================
float UMotionControllerComponent::GetParameterValue(FName InName, bool& bValueFound)
{
	if (InUseMotionController)
	{
		return InUseMotionController->GetCustomParameterValue(MotionSource, InName, bValueFound);
	}
	bValueFound = false;
	return 0.f;
}

//=============================================================================
FVector UMotionControllerComponent::GetHandJointPosition(int jointIndex, bool& bValueFound)
{
	FVector outPosition;
	if (InUseMotionController && InUseMotionController->GetHandJointPosition(MotionSource, jointIndex, outPosition))
	{
		bValueFound = true;
		return outPosition;
	}
	else
	{
		bValueFound = false;
		return FVector::ZeroVector;
	}
}

//=============================================================================
bool UMotionControllerComponent::GetLinearVelocity(FVector& OutLinearVelocity) const
{
	const IXRTrackingSystem* const TrackingSys = GEngine->XRSystem.Get();
	const FTransform TrackingToWorldTransform = TrackingSys ? TrackingSys->GetTrackingToWorldTransform() : FTransform::Identity;
	OutLinearVelocity = TrackingToWorldTransform.TransformVector(LinearVelocity);
	
	return bProvidedLinearVelocity;
}

//=============================================================================
bool UMotionControllerComponent::GetAngularVelocity(FRotator& OutAngularVelocity) const
{
	const IXRTrackingSystem* const TrackingSys = GEngine->XRSystem.Get();
	const FTransform TrackingToWorldTransform = TrackingSys ? TrackingSys->GetTrackingToWorldTransform() : FTransform::Identity;
	FVector WorldVector = TrackingToWorldTransform.TransformVector(AngularVelocityAsAxisAndLength);
	// Note: the rotator may contain rotations greater than 180 or 360 degrees, and some mathmatical operations (eg conversion to quaternion) would lose those.
	OutAngularVelocity = IMotionController::AngularVelocityAsAxisAndLengthToRotator(WorldVector);
	return bProvidedAngularVelocity;
}

//=============================================================================
bool UMotionControllerComponent::GetLinearAcceleration(FVector& OutLinearAcceleration) const
{
	const IXRTrackingSystem* const TrackingSys = GEngine->XRSystem.Get();
	const FTransform TrackingToWorldTransform = TrackingSys ? TrackingSys->GetTrackingToWorldTransform() : FTransform::Identity;
	OutLinearAcceleration = TrackingToWorldTransform.TransformVector(LinearAcceleration);
	
	return bProvidedLinearAcceleration;
}
