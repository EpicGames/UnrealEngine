// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotionDelayBuffer.h"
#include "HAL/PlatformTime.h" // for FPlatformTime::Seconds()
#include "Components/SceneComponent.h"
#include "RenderingThread.h" // for ENQUEUE_RENDER_COMMAND
#include "SceneView.h" // for FSceneViewFamily
#include "Logging/LogMacros.h" // for DEFINE_LOG_CATEGORY_STATIC
#include "IXRTrackingSystem.h"
#include "IMotionController.h"
#include "Templates/TypeHash.h"
#include "Features/IModularFeatures.h"
#include "GameFramework/WorldSettings.h"
#include "Engine/Engine.h" // for GEngine->XRSystem

DEFINE_LOG_CATEGORY_STATIC(LogMotionDelayBuffer, Log, All);

/* FMotionDelayTarget
 *****************************************************************************/

class FMotionDelayTarget : public TSharedFromThis<FMotionDelayTarget, ESPMode::ThreadSafe>
{
public:
	struct FMotionSource
	{
		int32 PlayerIndex;
		FName SourceId;

		bool operator==(const FMotionSource& Rhs) const;
		bool operator!=(const FMotionSource& Rhs) const;
	};
	FMotionDelayTarget(FMotionSource MotionSource);

public:
	FLateUpdateManager LateUpdate;
	FMotionSource MotionSource;
	TCircularHistoryBuffer<FVector> ScaleHistoryBuffer;
};

FMotionDelayTarget::FMotionDelayTarget(FMotionDelayTarget::FMotionSource InMotionSource)
	: MotionSource(InMotionSource)
{}

static uint32 GetTypeHash(const FMotionDelayTarget::FMotionSource& Lhs)
{
	return HashCombine( GetTypeHash(Lhs.PlayerIndex), GetTypeHash(Lhs.SourceId) );
}

bool FMotionDelayTarget::FMotionSource::operator==(const FMotionSource& Rhs) const
{
	return (PlayerIndex == Rhs.PlayerIndex) && (SourceId == Rhs.SourceId);
}

bool FMotionDelayTarget::FMotionSource::operator!=(const FMotionSource& Rhs) const
{
	return !operator==(Rhs);
}

/* MotionDelayService_Impl
 *****************************************************************************/

namespace MotionDelayService_Impl
{
	struct FPoseSample
	{
		FPoseSample() : Position(FVector::ZeroVector), Orientation(FRotator::ZeroRotator), TimeStamp(0.0) {}

		FVector  Position;
		FRotator Orientation;
		double   TimeStamp;
	};
	static TMap< FMotionDelayTarget::FMotionSource, TCircularHistoryBuffer<FPoseSample> > SourceSamples;

	// shared ref since the render thread needs reliable access to the late update buffer
	typedef TSharedRef<FMotionDelayTarget, ESPMode::ThreadSafe> FSharedDelayTarget;
	static TMap<TWeakObjectPtr<USceneComponent>, FSharedDelayTarget> DelayTargets;

	static TArray<TWeakPtr<FMotionDelayClient, ESPMode::ThreadSafe>> DelayClients;

	static bool bIsEnabled = false;
	static uint32 SharedBufferSizes = 0;
	static float DefaultWorldToMetersScale = 100.f;

	// frame ids to make sure we 'sync' and 'clean up' once per frame (triggered via, potentionally multiple, FMotionDelayClients)
	static uint32 LastSyncFrameId = UINT_MAX;
	static uint32 PostRenderCleanupId_RenderThread = UINT_MAX;

	static double FrameSyncTime = 0.0;

	template <typename F>
	static FORCEINLINE void ForEachClient(F Action);
	template <typename F>
	static FORCEINLINE void ForEachTarget(F Action);

	static int32 EstimateDelayIndex(uint32 MillisecDelay);
	static uint32 FindNeededBufferSize();
	static void RefreshDelayBufferSizes();
	static void RefreshMotionSources();

	static void SyncDelayBuffers(uint32 FrameId);
	static bool SampleDevicePose(const FMotionDelayTarget::FMotionSource& TargetSource, FPoseSample& PoseOut);
	static void PostRender_RenderThread(uint32 FrameId);
}

//------------------------------------------------------------------------------
template <typename F>
static FORCEINLINE void MotionDelayService_Impl::ForEachClient(F Action)
{
	checkSlow(IsInGameThread());

	for (int32 ClientIndex = DelayClients.Num() - 1; ClientIndex >= 0; --ClientIndex)
	{
		if (DelayClients[ClientIndex].IsValid())
		{
			Action(DelayClients[ClientIndex].Pin().Get());
		}
		else
		{
			DelayClients.RemoveAtSwap(ClientIndex);
		}
	}
}

//------------------------------------------------------------------------------
template <typename F>
static FORCEINLINE void MotionDelayService_Impl::ForEachTarget(F Action)
{
	checkSlow(IsInGameThread());

	TArray<TWeakObjectPtr<USceneComponent>> StaleTargets;
	for (auto& Target : DelayTargets)
	{
		if (Target.Key.IsValid())
		{
			Action(Target.Key.Get(), Target.Value.Get());
		}
		else
		{
			StaleTargets.Add(Target.Key);
		}
	}

	for (TWeakObjectPtr<USceneComponent>& StalePtr : StaleTargets)
	{
		DelayTargets.Remove(StalePtr);
	}
}

//------------------------------------------------------------------------------
static int32 MotionDelayService_Impl::EstimateDelayIndex(uint32 MillisecDelay)
{
	const float FixedFrameRateMs = 1000.f * 1.f / 90.f;
	return FMath::CeilToInt(MillisecDelay / FixedFrameRateMs);
}

//------------------------------------------------------------------------------
static uint32 MotionDelayService_Impl::FindNeededBufferSize()
{
	uint32 SharedSize = 0;
	auto FindMaxSize = [&SharedSize](FMotionDelayClient* Client)
	{
		const uint32 NeededSize = EstimateDelayIndex(Client->GetDesiredDelay()) + 1;
		SharedSize = FMath::Max(SharedSize, NeededSize);
	};
	ForEachClient(FindMaxSize);

	return SharedSize;
}

//------------------------------------------------------------------------------
static void MotionDelayService_Impl::RefreshDelayBufferSizes()
{
	const uint32 RequiredBufferSize = FindNeededBufferSize();
	if (SharedBufferSizes != RequiredBufferSize)
	{
		for (auto& Sampler : SourceSamples)
		{
			Sampler.Value.Resize(RequiredBufferSize);
		}

		auto ResizeTarget = [RequiredBufferSize](USceneComponent* /*Component*/, FMotionDelayTarget& Target)
		{
			Target.ScaleHistoryBuffer.Resize(RequiredBufferSize);
		};
		ForEachTarget(ResizeTarget);

		SharedBufferSizes = RequiredBufferSize;
	}
}

//------------------------------------------------------------------------------
static void MotionDelayService_Impl::RefreshMotionSources()
{
	TArray<FMotionDelayTarget::FMotionSource> UneededSources;
	UneededSources.Reserve(SourceSamples.Num());

	for (auto& Sampler : SourceSamples)
	{
		UneededSources.Add(Sampler.Key);
	}

	auto RemovedUsedSource = [&UneededSources](USceneComponent* /*Component*/, FMotionDelayTarget& Target)
	{
		UneededSources.Remove(Target.MotionSource);
	};
	ForEachTarget(RemovedUsedSource);

	for (const FMotionDelayTarget::FMotionSource& Source : UneededSources)
	{
		SourceSamples.Remove(Source);
	}
}

//------------------------------------------------------------------------------
static void MotionDelayService_Impl::SyncDelayBuffers(uint32 FrameId)
{
	checkSlow(IsInGameThread());

	if (FrameId != LastSyncFrameId)
	{
		RefreshDelayBufferSizes();

		for (auto& Sampler : SourceSamples)
		{
			FPoseSample NewSample;
			if (!SampleDevicePose(Sampler.Key, NewSample))
			{
				if (Sampler.Value.Num() > 0)
				{
					const FPoseSample& LastSample = Sampler.Value[0];
					NewSample.Position = LastSample.Position;
					NewSample.Orientation = LastSample.Orientation;
				}
			}

			Sampler.Value.Add(NewSample);
		}

		auto RemovedUsedSource = [](USceneComponent* Component, FMotionDelayTarget& Target)
		{
			UWorld* World = Component->GetWorld();
			const float WorldToMeteresScale = World ? World->GetWorldSettings()->WorldToMeters : DefaultWorldToMetersScale;

			FVector FrameScale(WorldToMeteresScale / DefaultWorldToMetersScale);
			FrameScale *= Component->GetComponentScale();

			Target.ScaleHistoryBuffer.Add(FrameScale);

			const USceneComponent* AttachParent = Component->GetAttachParent();
			FTransform ParentTransform = AttachParent ? AttachParent->GetComponentTransform() : FTransform::Identity;
			Target.LateUpdate.Setup(ParentTransform, Component, false);
		};
		ForEachTarget(RemovedUsedSource);

		LastSyncFrameId = FrameId;
		FrameSyncTime = FPlatformTime::Seconds();
	}
}

//------------------------------------------------------------------------------
static bool MotionDelayService_Impl::SampleDevicePose(const FMotionDelayTarget::FMotionSource& TargetSource, FPoseSample& PoseOut)
{
	PoseOut = FPoseSample();
	PoseOut.TimeStamp = FPlatformTime::Seconds();

	bool bSuccess = false;

	TArray<IMotionController*> MotionControllers = IModularFeatures::Get().GetModularFeatureImplementations<IMotionController>(IMotionController::GetModularFeatureName());
	for (IMotionController* MotionController : MotionControllers)
	{
		if (MotionController != nullptr)
		{
			if (MotionController->GetControllerOrientationAndPosition(TargetSource.PlayerIndex, TargetSource.SourceId, PoseOut.Orientation, PoseOut.Position, DefaultWorldToMetersScale))
			{
				bSuccess = true;
				break;
			}
		}
	}

	if (TargetSource.SourceId == IMotionController::HMDSourceId)
	{
		IXRTrackingSystem* TrackingSys = GEngine->XRSystem.Get();
		if (TrackingSys)
		{
			FQuat OrientationQuat;
			if (TrackingSys->GetCurrentPose(IXRTrackingSystem::HMDDeviceId, OrientationQuat, PoseOut.Position))
			{
				PoseOut.Orientation = OrientationQuat.Rotator();
				bSuccess = true;
			}
		}
	}

	return bSuccess;
}

/* FMotionDelayService
 *****************************************************************************/

//------------------------------------------------------------------------------
void FMotionDelayService::SetEnabled(bool bEnable)
{
	MotionDelayService_Impl::bIsEnabled = bEnable;
}

//------------------------------------------------------------------------------
bool FMotionDelayService::RegisterDelayTarget(USceneComponent* MotionControlledComponent, const int32 PlayerIndex, const FName SourceId)
{
	using namespace MotionDelayService_Impl;

	if (bIsEnabled)
	{
		FMotionDelayTarget::FMotionSource TargetSource;
		TargetSource.PlayerIndex = PlayerIndex;
		TargetSource.SourceId = SourceId;

		if (FSharedDelayTarget* DelayTarget = DelayTargets.Find(MotionControlledComponent))
		{
			if (DelayTarget->Get().MotionSource != TargetSource)
			{
				DelayTarget->Get().MotionSource = TargetSource;
				RefreshMotionSources();
			}
		}
		else
		{
			FSharedDelayTarget NewDelayTarget = DelayTargets.Add(MotionControlledComponent, MakeShareable(new FMotionDelayTarget(TargetSource)));
			if (SharedBufferSizes > 0)
			{
				NewDelayTarget->ScaleHistoryBuffer.Resize(SharedBufferSizes);
			}
		}

		if (!SourceSamples.Contains(TargetSource))
		{
			TCircularHistoryBuffer<FPoseSample>& SampleBuffer = SourceSamples.Add(TargetSource);
			if (SharedBufferSizes > 0)
			{
				SampleBuffer.Resize(SharedBufferSizes);
			}
		}
	}
	return bIsEnabled;
}

//------------------------------------------------------------------------------
void FMotionDelayService::RegisterDelayClient(TSharedRef<FMotionDelayClient, ESPMode::ThreadSafe> DelayClient)
{
	using namespace MotionDelayService_Impl;

	DelayClients.Add(DelayClient);
	RefreshDelayBufferSizes();
}

/* FMotionDelayClient
 *****************************************************************************/

namespace MotionDelayClient_Impl
{
	using namespace MotionDelayService_Impl;
	static void CalulateDelayTransform(uint32 DesiredDelay, const TCircularHistoryBuffer<FPoseSample>& SampleBuffer, const TCircularHistoryBuffer<FVector> ScaleBuffer, FTransform& TransformOut);
}

//------------------------------------------------------------------------------
static void MotionDelayClient_Impl::CalulateDelayTransform(uint32 DesiredDelay, const TCircularHistoryBuffer<MotionDelayService_Impl::FPoseSample>& SampleBuffer, const TCircularHistoryBuffer<FVector> ScaleBuffer, FTransform& TransformOut)
{
	checkSlow(IsInGameThread());

	int32 SampleAIndex = SampleBuffer.Num() - 1, SampleBIndex = INDEX_NONE;
	const int32 EstimatedIndex = FMath::Min(SampleAIndex, EstimateDelayIndex(DesiredDelay));

	const double DelaySeconds = DesiredDelay / 1000.f;
	// need to use the same current time for all delay transforms in a single frame, so that all
	// the calculated (interpolated) transforms match up relative to each other
	// NOTE: this is not what the current time will be by the time this renders, so you may have to fold in some extra delay
	const double CurrentTime = MotionDelayService_Impl::FrameSyncTime;

	int8 IncrementDir = 0;
	for (int32 SampleIndex = EstimatedIndex; SampleIndex < SampleBuffer.Num() && SampleIndex >= 0; SampleIndex += IncrementDir)
	{
		const FPoseSample& Sample = SampleBuffer[SampleIndex];

		const double TimeSince = CurrentTime - Sample.TimeStamp;
		if (IncrementDir == 0)
		{
			IncrementDir = (TimeSince >= DelaySeconds) ? -1 : +1;
		}
		else if ((-IncrementDir + 1) / 2 != (TimeSince >= DelaySeconds))
		{
			SampleAIndex = SampleIndex;
			SampleBIndex = SampleIndex - IncrementDir;
			break;
		}
	}

	auto MakeTransform = [&SampleBuffer, &ScaleBuffer](int32 Index)->FTransform
	{
		const FPoseSample& Sample = SampleBuffer[Index];
		const FVector& ComponentScale = ScaleBuffer[Index];

		return FTransform(Sample.Orientation, Sample.Position, ComponentScale);
	};

	if (SampleBIndex == INDEX_NONE)
	{
		UE_CLOGF(SampleBuffer.IsFull(), LogMotionDelayBuffer, Warning, "Not enough space in this motion delay buffer to accomedate the desired delay.");
		if (IncrementDir > 0)
		{
			TransformOut = MakeTransform(SampleAIndex);
		}
		else if (IncrementDir < 0)
		{
			TransformOut = MakeTransform(0);
		}
	}
	else
	{
		const FPoseSample& SampleA = SampleBuffer[SampleAIndex];
		const FPoseSample& SampleB = SampleBuffer[SampleBIndex];

		FTransform TransformA = MakeTransform(SampleAIndex);
		FTransform TransformB = MakeTransform(SampleBIndex);

		const float BlendAlpha = (float)((CurrentTime - SampleA.TimeStamp - DelaySeconds) / (SampleB.TimeStamp - SampleA.TimeStamp));
		TransformOut.Blend(TransformA, TransformB, BlendAlpha);
	}
}

 //------------------------------------------------------------------------------
FMotionDelayClient::FMotionDelayClient(const FAutoRegister& AutoRegister)
	: FSceneViewExtensionBase(AutoRegister)
{}

//------------------------------------------------------------------------------
void FMotionDelayClient::BeginRenderViewFamily(FSceneViewFamily& ViewFamily)
{
	using namespace MotionDelayService_Impl;

	// ensure that we take a sample for this frame
	SyncDelayBuffers(ViewFamily.FrameNumber);

	TArray<FTargetTransform> RenderTransforms;
	RenderTransforms.Reserve(DelayTargets.Num());

	TArray<USceneComponent*> ExemptTargets;
	GetExemptTargets(ExemptTargets);

	auto IsTargetExempt = [&ExemptTargets](const TWeakObjectPtr<USceneComponent>& DelayTarget)->bool
	{
		bool bExempt = !DelayTarget.IsValid();
		if (!bExempt)
		{
			USceneComponent* Target = DelayTarget.Get();
			bExempt = ExemptTargets.Contains(Target);
		}
		return bExempt;
	};

	const uint32 DesiredLatency = GetDesiredDelay();
	for (auto& Target : DelayTargets)
	{
		if (IsTargetExempt(Target.Key))
		{
			continue;
		}

		TCircularHistoryBuffer<FPoseSample>* SampleBuffer = SourceSamples.Find(Target.Value->MotionSource);
		if (ensure(SampleBuffer))
		{
			FTargetTransform RenderTransform;
			RenderTransform.DelayTarget = Target.Value;
			RenderTransform.DelayTransform = RenderTransform.RestoreTransform = Target.Key->GetRelativeTransform();

			MotionDelayClient_Impl::CalulateDelayTransform(DesiredLatency, *SampleBuffer, Target.Value->ScaleHistoryBuffer, RenderTransform.DelayTransform);

			RenderTransforms.Add(RenderTransform);
		}
	}

	TWeakPtr<FMotionDelayClient, ESPMode::ThreadSafe> ThisPtr = SharedThis(this);
	ENQUEUE_RENDER_COMMAND(MotionDelayClientSetup)(
		[ThisPtr, RenderTransforms](FRHICommandListImmediate& /*RHICmdList*/)
		{
			if (ThisPtr.IsValid())
			{
				ThisPtr.Pin()->TargetTransforms_RenderThread = RenderTransforms;
			}
		}
	);
}

//------------------------------------------------------------------------------
void FMotionDelayClient::PostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& ViewFamily)
{
	if (MotionDelayService_Impl::PostRenderCleanupId_RenderThread != ViewFamily.FrameNumber)
	{
		MotionDelayService_Impl::PostRenderCleanupId_RenderThread = ViewFamily.FrameNumber;
	}
}

//------------------------------------------------------------------------------
int32 FMotionDelayClient::GetPriority() const
{
	return -5;
}

//------------------------------------------------------------------------------
bool FMotionDelayClient::IsActiveThisFrame_Internal(const FSceneViewExtensionContext&) const
{
	using namespace MotionDelayService_Impl;

	const uint32 DesiredLatency = GetDesiredDelay();
	return DesiredLatency > 0 && DelayTargets.Num() > 0;
}

//------------------------------------------------------------------------------
void FMotionDelayClient::Apply_RenderThread(FSceneInterface* Scene)
{
	for (const FTargetTransform& Transform : TargetTransforms_RenderThread)
	{
		Transform.DelayTarget->LateUpdate.Apply_RenderThread(Scene, Transform.RestoreTransform, Transform.DelayTransform);
	}
}

//------------------------------------------------------------------------------
void FMotionDelayClient::Restore_RenderThread(FSceneInterface* Scene)
{
	for (const FTargetTransform& Transform : TargetTransforms_RenderThread)
	{
		Transform.DelayTarget->LateUpdate.Apply_RenderThread(Scene, Transform.DelayTransform, Transform.RestoreTransform);
	}
}

//------------------------------------------------------------------------------
bool FMotionDelayClient::FindDelayTransform(USceneComponent* Target, uint32 Delay, FTransform& TransformOut)
{
	using namespace MotionDelayService_Impl;

	bool bFoundTransform = false;
	if (FSharedDelayTarget* TargetData = DelayTargets.Find(Target))
	{
		TCircularHistoryBuffer<FPoseSample>* SampleBuffer = SourceSamples.Find((*TargetData)->MotionSource);
		if (ensure(SampleBuffer))
		{
			MotionDelayClient_Impl::CalulateDelayTransform(Delay, *SampleBuffer, (*TargetData)->ScaleHistoryBuffer, TransformOut);
			bFoundTransform = true;
		}
	}

	return bFoundTransform;
}
