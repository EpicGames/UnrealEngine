// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "IMotionController.h"
#include "InputCoreTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/IntRect.h"
#include "Math/Quat.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Misc/Guid.h"
#include "RHI.h"
#include "RHIResources.h"
#include "Trace/Detail/Channel.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "HeadMountedDisplayTypes.generated.h"

class FRHICommandListImmediate;
class UObject;
struct FFilterVertex;
struct FFrame;

class FHMDViewMesh
{
public:

	enum EHMDMeshType
	{
		MT_HiddenArea,
		MT_VisibleArea
	};

	HEADMOUNTEDDISPLAY_API FHMDViewMesh();
	HEADMOUNTEDDISPLAY_API ~FHMDViewMesh();

	bool IsValid() const
	{
		return NumTriangles > 0;
	}

	HEADMOUNTEDDISPLAY_API void BuildMesh(const FVector2D Positions[], uint32 VertexCount, EHMDMeshType MeshType);

	FBufferRHIRef VertexBufferRHI;
	FBufferRHIRef IndexBufferRHI;

	unsigned  NumVertices;
	unsigned  NumIndices;
	unsigned  NumTriangles;
};

HEADMOUNTEDDISPLAY_API DECLARE_LOG_CATEGORY_EXTERN(LogHMD, Log, All);
HEADMOUNTEDDISPLAY_API DECLARE_LOG_CATEGORY_EXTERN(LogLoadingSplash, Log, All);

UENUM()
namespace EOrientPositionSelector
{
	enum Type : int
	{
		Orientation UMETA(DisplayName = "Orientation"),
		Position UMETA(DisplayName = "Position"),
		OrientationAndPosition UMETA(DisplayName = "Orientation and Position")
	};
}

/**
* Specifies the type of tracking space origin we should use.  Be aware that not all devices support all Tracking Origin types. https://registry.khronos.org/OpenXR/specs/1.0/html/xrspec.html#reference-spaces
*/
UENUM()
namespace EHMDTrackingOrigin
{
	enum Type : int
	{
		View UMETA(ToolTip = "Previously sometimes used Eye space to query for the view transform, this space is fixed to the HMD, meaning that as the hmd moves this space moves relative to other spaces. This isn't used as a tracking origin.")
		, LocalFloor UMETA(ToolTip = "For standing stationary experiences. Typically centered around HMDs initial position either at app startup or device startup, with Z 0 set to match the floor as in the Stage Space. Falls back to local.")
		, Local UMETA(ToolTip = "For seated experiences. Always Supported. Typically centered around the HMDs initial position either at app startup or device startup.  Useful for seated experiences. Previously called Eye Space.  ")
		, Stage UMETA(ToolTip = "For walking-around experiences.  The origin will be at floor level and typically within a defined play areas who's bounds will be available. Falls back to local.")
		, CustomOpenXR UMETA(ToolTip = "Custom OpenXR tracking space of some kind. You cannot set this space explictly, it is automatically used by some platform plugin extensions.")
	};
}

/**
* Stores if the user is wearing the HMD or not. For HMDs without a sensor to detect the user wearing it, the state defaults to Unknown.
*/
UENUM()
namespace EHMDWornState
{
	enum Type : int
	{
		Unknown UMETA(DisplayName = "Unknown"),
		Worn UMETA(DisplayName = "Worn"),
		NotWorn UMETA(DisplayName = "Not Worn"),
	};
}


/**
* Enumeration of results from Connecting to Remote XR device
*/
UENUM(BlueprintType)
namespace EXRDeviceConnectionResult
{
	enum Type : int
	{
		NoTrackingSystem,
		FeatureNotSupported,
		NoValidViewport,
		MiscFailure,
		Success
	};
}

/**
* Flags to better inform the user about specifics of the underlying XR system
*/
UENUM(BlueprintType)
namespace EXRSystemFlags
{
	enum Type : int
	{
		NoFlags       = 0x00 UMETA(Hidden),
		IsAR          = 0x01,
		IsTablet      = 0x02,
		IsHeadMounted = 0x04,
		SupportsHandTracking = 0x08,
	};
}
/**
* The Spectator Screen Mode controls what the non-vr video device displays on platforms that support one.
* Not all modes are universal.
* Modes SingleEyeCroppedToFill, Texture, and MirrorPlusTexture are supported on all.
*/
UENUM()
enum class ESpectatorScreenMode : uint8
{
	Disabled				UMETA(DisplayName = "Disabled"),
	SingleEyeLetterboxed	UMETA(DisplayName = "SingleEyeLetterboxed"),
	Undistorted				UMETA(DisplayName = "Undistorted"),
	Distorted				UMETA(DisplayName = "Distorted"),
	SingleEye				UMETA(DisplayName = "SingleEye"),
	SingleEyeCroppedToFill	UMETA(DisplayName = "SingleEyeCroppedToFill"),
	Texture					UMETA(DisplayName = "Texture"),
	TexturePlusEye			UMETA(DisplayName = "TexturePlusEye"),
};
const uint8 ESpectatorScreenModeFirst = (uint8)ESpectatorScreenMode::Disabled;
const uint8 ESpectatorScreenModeLast = (uint8)ESpectatorScreenMode::TexturePlusEye;

struct FSpectatorScreenModeTexturePlusEyeLayout
{
	FSpectatorScreenModeTexturePlusEyeLayout()
		: EyeRectMin(0.0f, 0.0f)
		, EyeRectMax(1.0f, 1.0f)
		, TextureRectMin(0.125f, 0.125f)
		, TextureRectMax(0.25f, 0.25f)
		, bDrawEyeFirst(true)
		, bUseAlpha(false)
		, bClearBlack(false)
	{}

	FSpectatorScreenModeTexturePlusEyeLayout(FVector2D InEyeRectMin, FVector2D InEyeRectMax, FVector2D InTextureRectMin, FVector2D InTextureRectMax, bool InbDrawEyeFirst, bool InbClearBlack, bool InbUseAlpha)
		: EyeRectMin(InEyeRectMin)
		, EyeRectMax(InEyeRectMax)
		, TextureRectMin(InTextureRectMin)
		, TextureRectMax(InTextureRectMax)
		, bDrawEyeFirst(InbDrawEyeFirst)
		, bUseAlpha(InbUseAlpha)
		, bClearBlack(InbClearBlack)
	{}

	bool IsValid() const 
	{
		bool bValid = true;
		if ((EyeRectMax.X <= EyeRectMin.X) || (EyeRectMax.Y <= EyeRectMin.Y))
		{
			UE_LOGF(LogHMD, Warning, "SpectatorScreenModeTexturePlusEyeLayout EyeRect is invalid!  Max is not greater than Min in some dimension.");
			bValid = false;
		}
		if ((TextureRectMax.X <= TextureRectMin.X) || (TextureRectMax.Y <= TextureRectMin.Y))
		{
			UE_LOGF(LogHMD, Warning, "SpectatorScreenModeTexturePlusEyeLayout TextureRect is invalid!  Max is not greater than Min in some dimension.");
			bValid = false;
		}
		if (EyeRectMin.X > 1.0f || EyeRectMin.X < 0.0f ||
			EyeRectMin.Y > 1.0f || EyeRectMin.Y < 0.0f ||
			EyeRectMax.X > 1.0f || EyeRectMax.X < 0.0f ||
			EyeRectMax.Y > 1.0f || EyeRectMax.Y < 0.0f
			)
		{
			UE_LOGF(LogHMD, Warning, "SpectatorScreenModeTexturePlusEyeLayout EyeRect is invalid!  All dimensions must be in 0-1 range.");
			bValid = false;
		}

		if (TextureRectMin.X > 1.0f || TextureRectMin.X < 0.0f ||
			TextureRectMin.Y > 1.0f || TextureRectMin.Y < 0.0f ||
			TextureRectMax.X > 1.0f || TextureRectMax.X < 0.0f ||
			TextureRectMax.Y > 1.0f || TextureRectMax.Y < 0.0f
			 )
		{
			UE_LOGF(LogHMD, Warning, "SpectatorScreenModeTexturePlusEyeLayout TextureRect is invalid!  All dimensions must be in 0-1 range.");
			bValid = false;
		}
		return bValid;
	}

	FIntRect GetScaledEyeRect(int SizeX, int SizeY) const
	{
		return FIntRect(FIntRect::IntType(EyeRectMin.X * SizeX), FIntRect::IntType(EyeRectMin.Y * SizeY), FIntRect::IntType(EyeRectMax.X * SizeX), FIntRect::IntType(EyeRectMax.Y * SizeY));
	}

	FIntRect GetScaledTextureRect(int SizeX, int SizeY) const
	{
		return FIntRect(FIntRect::IntType(TextureRectMin.X * SizeX), FIntRect::IntType(TextureRectMin.Y * SizeY), FIntRect::IntType(TextureRectMax.X * SizeX), FIntRect::IntType(TextureRectMax.Y * SizeY));
	}

	FVector2D EyeRectMin;
	FVector2D EyeRectMax;
	FVector2D TextureRectMin;
	FVector2D TextureRectMax;
	bool bDrawEyeFirst;
	bool bUseAlpha;
	bool bClearBlack;
};

UENUM(BlueprintType)
enum class EXRTrackedDeviceType : uint8
{
	/** Represents a head mounted display */
	HeadMountedDisplay,
	/** Represents a controller */
	Controller,
	/** Represents a static tracking reference device, such as a Lighthouse or tracking camera */
	TrackingReference,
	/** Represents trackers, such as a Vive tracker */
	Tracker,
	/** Misc. device types, for future expansion */
	Other,
	/** DeviceId is invalid */
	Invalid = (uint8)-2 UMETA(Hidden),
	/** Pass to EnumerateTrackedDevices to get all devices regardless of type */
	Any = (uint8)-1,
};



/**
 * Transforms that are tracked on the hand.
 * Matches the enums from WMR to make it a direct mapping
 */
UENUM(BlueprintType)
enum class EHandKeypoint : uint8
{
	Palm,
	Wrist,
	ThumbMetacarpal,
	ThumbProximal,
	ThumbDistal,
	ThumbTip,
	IndexMetacarpal,
	IndexProximal,
	IndexIntermediate,
	IndexDistal,
	IndexTip,
	MiddleMetacarpal,
	MiddleProximal,
	MiddleIntermediate,
	MiddleDistal,
	MiddleTip,
	RingMetacarpal,
	RingProximal,
	RingIntermediate,
	RingDistal,
	RingTip,
	LittleMetacarpal,
	LittleProximal,
	LittleIntermediate,
	LittleDistal,
	LittleTip
};

const int32 EHandKeypointCount = static_cast<int32>(EHandKeypoint::LittleTip) + 1;

UCLASS(MinimalAPI)
class UHandKeypointConversion : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Interpret a HandKeypoint as an int input, useful for indexing into arrays of hand keypoint data. */
	UFUNCTION(BlueprintPure, BlueprintInternalUseOnly, meta = (CompactNodeTitle = "->", BlueprintAutocast))
	static int32 Conv_HandKeypointToInt32(EHandKeypoint input)
	{
		return static_cast<int32>(input);
	}
};

UENUM(BlueprintType)
enum class EXRVisualType : uint8
{
	Controller,
	Hand
};

/**
 * Used to get unreal world space or tracking space positions in XR.
 * Working only with unreal world space coordinates is almost always simpler and should be generally preferred.
 * There are specific cases where raw XRTrackingSpace coordinates may be useful, particularly in apps where world scale is not 1:1 or where it changes.
 */
UENUM(BlueprintType)
enum class EXRSpaceType : uint8
{
	/* The unreal coordinate system.  Affected by world scaling and the TrackingToWorldTransform.*/
	UnrealWorldSpace,
	/* The coordinate system the XR Device is tracking itself in.  Should be fixed relative to the real world over short timescales, not affected by world scaling or the TrackingToWorldTransoform.  May change relationship to the physical world through recentering or similar events.*/
	XRTrackingSpace
};

/**
 * Controllers, due to their various shapes, might have different natural usages.
 */
UENUM(BlueprintType)
enum class EXRControllerPoseType : uint8
{
	/* A pose meant to represent pointing.*/
	Aim,
	/* A poose meant to represent holding a stick.*/
	Grip,
	/* A pose meant to represent holding an item in the palm of the hand.*/
	Palm
};

USTRUCT(BlueprintType)
struct FXRHMDData
{
	GENERATED_USTRUCT_BODY();

	UPROPERTY(BlueprintReadOnly, Category = "XR")
	bool bValid = false;
	UPROPERTY(BlueprintReadOnly, Category = "XR")
	FName DeviceName;
	UPROPERTY(BlueprintReadOnly, Category = "XR")
	FGuid ApplicationInstanceID;

	UPROPERTY(BlueprintReadOnly, Category = "XR")
	ETrackingStatus TrackingStatus = ETrackingStatus::NotTracked;

	UPROPERTY(BlueprintReadOnly, Category = "XR")
	FVector Position = FVector(0.0f);;
	UPROPERTY(BlueprintReadOnly, Category = "XR")
	FQuat Rotation = FQuat(EForceInit::ForceInitToZero);
};

USTRUCT(BlueprintType)
struct FXRMotionControllerState
{
	GENERATED_USTRUCT_BODY();

	// The state is valid if a pose has ever been provided.
	UPROPERTY(BlueprintReadOnly, Category = "XR")
	bool bValid = false;
	UPROPERTY(BlueprintReadOnly, Category = "XR")
	FName DeviceName;
	UPROPERTY(BlueprintReadOnly, Category = "XR")
	FGuid ApplicationInstanceID;

	UPROPERTY(BlueprintReadOnly, Category = "XR")
	EXRSpaceType XRSpaceType = EXRSpaceType::UnrealWorldSpace;

	UPROPERTY(BlueprintReadOnly, Category = "XR")
	EControllerHand Hand = EControllerHand::Left;

	// If a controller pose has been provided this frame the TrackingStatus will be Tracked.
	UPROPERTY(BlueprintReadOnly, Category = "XR")
	ETrackingStatus TrackingStatus = ETrackingStatus::NotTracked;

	UPROPERTY(BlueprintReadOnly, Category = "XR")
	EXRControllerPoseType XRControllerPoseType = EXRControllerPoseType::Grip;

	UPROPERTY(BlueprintReadOnly, Category = "XR")
	FVector ControllerLocation = FVector(0.0f);

	UPROPERTY(BlueprintReadOnly, Category = "XR")
	FQuat ControllerRotation = FQuat(EForceInit::ForceInitToZero);;

	// These are used by the XRVisualizationFunctionLibrary, and are not exposed to blueprint.
	UPROPERTY()
	FVector GripUnrealSpaceLocation = FVector(0.0f);
	UPROPERTY()
	FQuat GripUnrealSpaceRotation = FQuat(EForceInit::ForceInitToZero);;
};

USTRUCT(BlueprintType)
struct FXRHandTrackingState
{
	GENERATED_USTRUCT_BODY();

	// The state is valid if poses have ever been provided.
	UPROPERTY(BlueprintReadOnly, Category = "XR")
	bool bValid = false;
	UPROPERTY(BlueprintReadOnly, Category = "XR")
	FName DeviceName;
	UPROPERTY(BlueprintReadOnly, Category = "XR")
	FGuid ApplicationInstanceID;

	UPROPERTY(BlueprintReadOnly, Category = "XR")
	EXRSpaceType XRSpaceType = EXRSpaceType::UnrealWorldSpace;

	UPROPERTY(BlueprintReadOnly, Category = "XR")
	EControllerHand Hand = EControllerHand::Left;

	UPROPERTY(BlueprintReadOnly, Category = "XR")
	ETrackingStatus TrackingStatus = ETrackingStatus::NotTracked;

	// The indices of this array are the values of EHandKeypoint (Palm, Wrist, ThumbMetacarpal, etc).
	UPROPERTY(BlueprintReadOnly, Category = "XR")
	TArray<FVector> HandKeyLocations;

	// The indices of this array are the values of EHandKeypoint (Palm, Wrist, ThumbMetacarpal, etc).
	UPROPERTY(BlueprintReadOnly, Category = "XR")
	TArray<FQuat> HandKeyRotations;

	// The indices of this array are the values of EHandKeypoint (Palm, Wrist, ThumbMetacarpal, etc).
	UPROPERTY(BlueprintReadOnly, Category = "XR")
	TArray<float> HandKeyRadii;
};