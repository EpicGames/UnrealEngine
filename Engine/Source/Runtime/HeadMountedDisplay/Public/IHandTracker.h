// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Features/IModularFeature.h"
#include "Math/Quat.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "UObject/NameTypes.h"

enum class EControllerHand : uint8;
enum class EHandKeypoint : uint8;

/**
 */

class IHandTracker : public IModularFeature
{
public:
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName("HandTracker");
		return FeatureName;
	}

	/**
	* Returns the device type of the controller.
	*
	* @return	Device type of the controller.
	*/
	virtual FName GetHandTrackerDeviceTypeName() const = 0;


	/**
	 * Returns true if hand tracking is available and tracking.
	 *
	 * @return			true/false
	 */
	virtual bool IsHandTrackingStateValid() const = 0;

	/**
	 * Get the transform and radius (or 0 if radius is not available on this platform) for the given hand keypoint.
	 *
	 * @return			true if data was fetched
	 */
	virtual bool GetKeypointState(EControllerHand Hand, EHandKeypoint Keypoint, FTransform& OutTransform, float& OutRadius) const = 0;

	virtual bool GetAllKeypointStates(EControllerHand Hand, TArray<FVector>& OutPositions, TArray<FQuat>& OutRotations, TArray<float>& OutRadii, bool& OutIsTracked) const = 0;

	virtual bool HasHandMeshData() const
	{
		return false;
	}

	virtual bool GetHandMeshData(EControllerHand Hand, TArray<FVector>& OutVertices, TArray<FVector>& OutNormals, TArray<int32>& OutIndices, FTransform& OutHandMeshTransform) const
	{
		return false;
	}
};
