// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "HeadMountedDisplayTypes.h"
#include "StereoRendering.h"
#include "LateUpdateManager.h"
#include "SceneInterface.h"

class FSceneInterface;
class UCanvas;
struct FPostProcessSettings;
struct FWorldContext;
class UTexture;
class FSceneViewFamily;
class FSceneView;
class FRHICommandListImmediate;
class FTexture;

struct FHeadMountedDisplayPassContext
{
	FHeadMountedDisplayPassContext(FRHICommandListImmediate& InRHICmdList, const FSceneView& InView)
		: RHICmdList(InRHICmdList)
		, View(InView)
	{}

	FRHICommandListImmediate& RHICmdList;
	const FSceneView& View;
};

/**
 * HMD device interface
 */

class IHeadMountedDisplay : public IModuleInterface
{

public:
	HEADMOUNTEDDISPLAY_API IHeadMountedDisplay();

	/**
	 * Returns true if HMD is currently connected.  It may or may not be in use.
	 */
	virtual bool IsHMDConnected() = 0;

	/**
	 * Whether or not switching to stereo is enabled; if it is false, then EnableStereo(true) will do nothing.
	 */
	virtual bool IsHMDEnabled() const = 0;

	/**
	* Returns EHMDWornState::Worn if we detect that the user is wearing the HMD, EHMDWornState::NotWorn if we detect the user is not wearing the HMD, and EHMDWornState::Unknown if we cannot detect the state.
	*/
	virtual EHMDWornState::Type GetHMDWornState() { return EHMDWornState::Unknown; };

	/**
	 * Enables or disables switching to stereo.
	 */
	virtual void EnableHMD(bool bEnable = true) = 0;

	/**
	 * Retrieves the HMD name.
	 */
	virtual FName GetHMDName() const = 0;

	struct MonitorInfo
	{
		FString MonitorName;
		size_t  MonitorId;
		int		DesktopX, DesktopY;
		int		ResolutionX, ResolutionY;
		int		WindowSizeX, WindowSizeY;
		bool	bShouldTestResolution;

		MonitorInfo() : MonitorId(0)
			, DesktopX(0)
			, DesktopY(0)
			, ResolutionX(0)
			, ResolutionY(0)
			, WindowSizeX(0)
			, WindowSizeY(0)
			, bShouldTestResolution(false)
		{
		}

	};

	/**
	 * Get the name or id of the display to output for this HMD.
	 */
	virtual bool	GetHMDMonitorInfo(MonitorInfo&) = 0;

	/**
	 * Calculates the FOV, based on the screen dimensions of the device. Original FOV is passed as params.
	 */
	virtual void	GetFieldOfView(float& InOutHFOVInDegrees, float& InOutVFOVInDegrees) const = 0;

	/**
	 * Sets near and far clipping planes (NCP and FCP) for the HMD.
	 *
	 * @param NCP				(in) Near clipping plane, in centimeters
	 * @param FCP				(in) Far clipping plane, in centimeters
	 */
	virtual void	SetClippingPlanes(float NCP, float FCP) {}

	/**
	 * Returns eye render params, used from PostProcessHMD, RenderThread.
	 */
	virtual void	GetEyeRenderParams_RenderThread(const struct FHeadMountedDisplayPassContext& Context, FVector2D& EyeToSrcUVScaleValue, FVector2D& EyeToSrcUVOffsetValue) const {}

	/**
	 * Accessors to modify the interpupillary distance (meters)
	 */
	virtual void	SetInterpupillaryDistance(float NewInterpupillaryDistance) = 0;
	virtual float	GetInterpupillaryDistance() const = 0;

	/**
	 * Whether HMDDistortion post processing is enabled or not
	 */
	virtual bool GetHMDDistortionEnabled(EShadingPath ShadingPath) const = 0;

	// Are we outputting so a Spectator Screen now.
	virtual bool IsSpectatorScreenActive() const { return false; }

	/**
	* Return a pointer to the SpectatorScreenController for the HMD if supported, else null.
	* The controller is owned by the HMD, and will be destroyed when the HMD is destroyed.
	*/
	virtual class ISpectatorScreenController* GetSpectatorScreenController() { return nullptr; }
	virtual class ISpectatorScreenController const* GetSpectatorScreenController() const { return nullptr; }

	/**
	* When implemented, creates a new post process node to provide platform-specific HMD distortion.
	*/
	virtual void CreateHMDPostProcessPass_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const struct FHMDDistortionInputs& Inputs, struct FScreenPassTexture& OutPass) const {}

public:

	/**
	 * Gets the current pixel density setting.
	 */
	virtual float GetPixelDenity() const { check(IsInGameThread() || IsInRenderingThread()); return 1.0f; }

	/**
	 * Sets the pixel density. This may cause render target resizing.
	 */
	virtual void SetPixelDensity(const float NewDensity) { };

	/**
	* Gets the ideal render target size for the device. See xr.SecondaryScreenPercentage.HMDRenderTarget description.
	*/
	virtual FIntPoint GetIdealRenderTargetSize() const { check(IsInGameThread() || IsInRenderingThread()); return FIntPoint(); }

	/**
	* Gets the ideal render target size for the debug canvas on the device.
	*/
	virtual FIntPoint GetIdealDebugCanvasRenderTargetSize() const { return FIntPoint(1024, 1024); }

	/**
	 * Gets the scaling factor, applied to the post process warping effect
	 */
	virtual float GetDistortionScalingFactor() const { return 0; }

	/**
	 * Gets the offset (in clip coordinates) from the center of the screen for the lens position
	 */
	virtual float GetLensCenterOffset() const { return 0; }

	/**
	 * Gets the barrel distortion shader warp values for the device
	 */
	virtual void GetDistortionWarpValues(FVector4& K) const  { }

	/**
	 * Returns 'false' if chromatic aberration correction is off.
	 */
	virtual bool IsChromaAbCorrectionEnabled() const = 0;

	/**
	 * Gets the chromatic aberration correction shader values for the device.
	 * Returns 'false' if chromatic aberration correction is off.
	 */
	virtual bool GetChromaAbCorrectionValues(FVector4& K) const  { return false; }

	/**
	* @return true if a hidden area mesh is available for the device.
	*/
	virtual bool HasHiddenAreaMesh() const { return false; }

	/**
	* @return true if a visible area mesh is available for the device.
	*/
	virtual bool HasVisibleAreaMesh() const { return false; }

	/**
	* Optional method to draw a view's hidden area mesh where supported.
	* This can be used to avoid rendering pixels which are not included as input into the final distortion pass.
	*/
	virtual void DrawHiddenAreaMesh(class FRHICommandList& RHICmdList, int32 ViewIndex) const {};

	/**
	* Optional method to draw a view's hidden area mesh where supported.
	* This can be used to avoid rendering pixels which are not included as input into the final distortion pass.
	* Supports instanced rendering.
	*/
	virtual void DrawHiddenAreaMesh(class FRHICommandList& RHICmdList, int32 ViewIndex, int32 InstanceCount) const {};

	/**
	* Optional method to draw a view's visible area mesh where supported.
	* This can be used instead of a full screen quad to avoid rendering pixels which are not included as input into the final distortion pass.
	*/
	virtual void DrawVisibleAreaMesh(class FRHICommandList& RHICmdList, int32 ViewIndex) const {};

	/**
	* Optional method to draw a view's visible area mesh where supported.
	* This can be used instead of a full screen quad to avoid rendering pixels which are not included as input into the final distortion pass.
	* Supports instanced rendering.
	*/
	virtual void DrawVisibleAreaMesh(class FRHICommandList& RHICmdList, int32 ViewIndex, int32 InstanceCount) const {};

	virtual void DrawDistortionMesh_RenderThread(struct FHeadMountedDisplayPassContext& Context, const FIntPoint& TextureSize) {}

	/**
	 * This method is able to change screen settings right before any drawing occurs. 
	 * It is called at the beginning of UGameViewportClient::Draw() method.
	 * We might remove this one as UpdatePostProcessSettings should be able to capture all needed cases
	 */
	virtual void UpdateScreenSettings(const class FViewport* InViewport) {}	

	/** 
	 * Additional optional distortion rendering parameters
	 * @todo:  Once we can move shaders into plugins, remove these!
	 */	
	virtual FTexture* GetDistortionTextureLeft() const {return NULL;}
	virtual FTexture* GetDistortionTextureRight() const {return NULL;}
	virtual FVector2D GetTextureOffsetLeft() const {return FVector2D::ZeroVector;}
	virtual FVector2D GetTextureOffsetRight() const {return FVector2D::ZeroVector;}
	virtual FVector2D GetTextureScaleLeft() const {return FVector2D::ZeroVector;}
	virtual FVector2D GetTextureScaleRight() const {return FVector2D::ZeroVector;}
	virtual const float* GetRedDistortionParameters() const { return nullptr; }
	virtual const float* GetGreenDistortionParameters() const { return nullptr; }
	virtual const float* GetBlueDistortionParameters() const { return nullptr; }

	virtual bool NeedsUpscalePostProcessPass()  { return false; }

	/**
	 * Record analytics
	 */
	virtual void RecordAnalytics() {}

	/**
	 * Returns true, if the App is using VR focus. This means that the app may handle lifecycle events differently from
	 * the regular desktop apps. In this case, FCoreDelegates::ApplicationWillEnterBackgroundDelegate and FCoreDelegates::ApplicationHasEnteredForegroundDelegate
	 * reflect the state of VR focus (either the app should be rendered in HMD or not).
	 */
	HEADMOUNTEDDISPLAY_API virtual bool DoesAppUseVRFocus() const;

	/**
	 * Returns true, if the app has VR focus, meaning if it is rendered in the HMD.
	 */
	HEADMOUNTEDDISPLAY_API virtual bool DoesAppHaveVRFocus() const;

	/**
	 * If true, scene rendering should be skipped.
	 */
	virtual bool IsRenderingPaused() const { return false; }

	/**
	 * Set the multiply and offset colors for composited images in the HMD
	 */
	virtual bool SetColorScaleAndBias(FLinearColor ColorScale, FLinearColor ColorBias) { return false;  };
};
