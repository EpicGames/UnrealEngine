// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Misc/ConfigCacheIni.h"
#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"

//#include "IHeadMountedDisplayVulkanExtensions.h"
class IHeadMountedDisplayVulkanExtensions;
class IXRTrackingSystem;

/**
 * The public interface of the HeadmountedDisplay Module
 */
class IHeadMountedDisplayModule : public IModuleInterface, public IModularFeature
{
public:
	static FName GetModularFeatureName()
	{
		static FName HMDFeatureName = FName("HMD");
		return HMDFeatureName;
	}

	/** Returns the key into the HMDPluginPriority section of the config file for this module */
	virtual FString GetModuleKeyName() const = 0;
	/** Returns an array of alternative ini/config names for this module (helpful if the module's name changes, so we can have back-compat) */
	virtual void GetModuleAliases(TArray<FString>& AliasesOut) const {}
	
	/** Returns the priority of this module from INI file configuration */
	float GetModulePriority() const
	{
		TArray<FString> ModuleAliases;
		GetModuleAliases(ModuleAliases);

		FString DefaultName = GetModuleKeyName();
		if (DefaultName.IsEmpty())
		{
			ModuleAliases.Add(TEXT("Default"));
		}
		else
		{
			// Search for aliases first. This favors old module names, and ensures 
			// that overrides in project specific ini files get found (not just the one in BaseEngine.ini)
			ModuleAliases.Add(DefaultName);
		}

		float ModulePriority = 0.f;
		for (const FString& KeyName : ModuleAliases)
		{
			if (GConfig->GetFloat(TEXT("HMDPluginPriority"), *KeyName, ModulePriority, GEngineIni))
			{
				break;
			}
		}	
		return ModulePriority;
	}

	/** Sorting method for which plug-in should be given priority */
	struct FCompareModulePriority
	{
		bool operator()(IHeadMountedDisplayModule& A, IHeadMountedDisplayModule& B) const
		{
			return A.GetModulePriority() > B.GetModulePriority();
		}
	};

	/**
	 * Singleton-like access to IHeadMountedDisplayModule
	 *
	 * @return Returns reference to the highest priority IHeadMountedDisplayModule module
	 */
	static inline IHeadMountedDisplayModule& Get()
	{
		check(IsInGameThread());
		TArray<IHeadMountedDisplayModule*> HMDModules = IModularFeatures::Get().GetModularFeatureImplementations<IHeadMountedDisplayModule>(GetModularFeatureName());
		HMDModules.Sort(FCompareModulePriority());
		return *HMDModules[0];
	}

	/**
	 * Checks to see if there exists a module registered as an HMD.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if there exists a module registered as an HMD.
	 */
	static inline bool IsAvailable()
	{
		check(IsInGameThread());
		return IModularFeatures::Get().IsModularFeatureAvailable(GetModularFeatureName());
	}

	/**
	* Register module as an HMD on startup.
	*/
	virtual void StartupModule() override
	{
		IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);
	}

	/**
	* Optionally pre-initialize the HMD module.  Return false on failure.
	*/
	virtual bool PreInit() { return true; }

	/**
	 * Test to see whether HMD is connected.  Used to guide which plug-in to select.
	 */
	virtual bool IsHMDConnected() { return false; }

	/**
	 * Get LUID of graphics adapter where the HMD was last connected.
	 *
	 * @TODO  currently, for mac, GetGraphicsAdapterLuid() is used to return a device index (how the function
	 *        "GetGraphicsAdapter" used to work), not a ID... eventually we want the HMD module to return the
	 *        MTLDevice's registryID, but we cannot fully handle that until we drop support for 10.12
	 *  NOTE: this is why we  use -1 as a sentinel value representing "no device" (instead of 0, which is used in the LUID case)
	 */
	virtual uint64 GetGraphicsAdapterLuid() 
	{ 
#if PLATFORM_MAC
		return (uint64)-1;
#else
		return 0;
#endif
	}

	/**
	 * Get name of audio input device where the HMD was last connected
	 */
	virtual FString GetAudioInputDevice() { return FString(); }

	/**
	 * Get name of audio output device where the HMD was last connected
	 */
	virtual FString GetAudioOutputDevice() { return FString(); }

	/**
	 * Get XR system name
	 */
	virtual FString GetDeviceSystemName() { return FString(); }

	/**
	 * Attempts to create a new head tracking device interface
	 *
	 * @return	Interface to the new head tracking device, if we were able to successfully create one
	 */
	virtual TSharedPtr< class IXRTrackingSystem, ESPMode::ThreadSafe > CreateTrackingSystem() = 0;

	/**
	 * Extensions:
	 * If the HMD supports the various extensions listed below, it should return a valid pointer to an implementation contained within it.
	 */
	virtual TSharedPtr< IHeadMountedDisplayVulkanExtensions, ESPMode::ThreadSafe > GetVulkanExtensions() { return nullptr; }

	/**
	* Indicates that the device we're currently running does not support a spectator view.
	* This will only be called once at initialization and should only return a result based for the current device the engine is running on.
	*/
	virtual bool IsStandaloneStereoOnlyDevice() { return false; }
};
