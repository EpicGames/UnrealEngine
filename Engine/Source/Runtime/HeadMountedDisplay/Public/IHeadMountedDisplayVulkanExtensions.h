// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

/**
* Query Vulkan extensions required by the HMD.
*
* Knowledge of these extensions is required during Vulkan RHI initialization.
*/
class IHeadMountedDisplayVulkanExtensions
{
public:
	virtual bool GetVulkanInstanceExtensionsRequired(TArray<const ANSICHAR*>& Out) = 0;
	virtual bool GetVulkanDeviceExtensionsRequired(struct VkPhysicalDevice_T *pPhysicalDevice, TArray<const ANSICHAR*>& Out) = 0;

	/**
	* If true and VK_PRESENT_MODE_MAILBOX_KHR is not available, VK_PRESENT_MODE_IMMEDIATE_KHR will be preferred over VK_PRESENT_MODE_FIFO_KHR
	* when creating the swap chain for the spectator window.
	*/
	HEADMOUNTEDDISPLAY_API virtual bool ShouldDisableVulkanVSync() const;
};
