// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h" // for FName
#include "HAL/Platform.h"
#include "Templates/TypeHash.h" // for HashCombine()
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"

#include "IIdentifiableXRDevice.generated.h"

class IXRSystemIdentifier
{
public:
	/**
	 * Returns a unique identifier that's supposed to represent the third party 
	 * system that this object is part of (Vive, Oculus, etc.).
	 *
	 * @return  A name unique to the system which this object belongs to.
	 */
	virtual FName GetSystemName() const = 0;
};

/** 
 * Generic device identifier interface
 *
 * This class is meant to provide a way to identify and distinguish
 * XR devices across various XR systems in a platform-agnostic way. 
 *
 * Additionally, it can be used to tie various IModularFeature device interfaces 
 * together. For example, if you have separate IXRTrackingSystem and IXRSystemAssets
 * interfaces which both reference the same devices, then this base class gives 
 * you a way to communicate between the two.
 */
class IIdentifiableXRDevice : public IXRSystemIdentifier
{
public:
	/**
	 * Returns a unique identifier that can be used to reference this device 
	 * within the system is belongs to.
	 *
	 * @return  A numerical identifier, unique to this device (not guaranteed to be unique outside of the system this belongs to).
	 */
	virtual int32 GetSystemDeviceId() const = 0;

	/** Combines the different aspects of IIdentifiableXRDevice to produce a unique identifier across all XR systems */
	friend uint32 GetTypeHash(const IIdentifiableXRDevice& XRDevice)
	{
		const uint32 DomainHash = GetTypeHash(XRDevice.GetSystemName());
		const uint32 DeviceHash = GetTypeHash(XRDevice.GetSystemDeviceId());
		return HashCombine(DomainHash, DeviceHash);
	}
};

USTRUCT(BlueprintType)
struct FXRDeviceId
{
	GENERATED_USTRUCT_BODY()

public:
	HEADMOUNTEDDISPLAY_API FXRDeviceId();
	HEADMOUNTEDDISPLAY_API FXRDeviceId(IIdentifiableXRDevice* DeviceId);
	HEADMOUNTEDDISPLAY_API FXRDeviceId(IXRSystemIdentifier* OwningSystem, const int32 DeviceId);

	HEADMOUNTEDDISPLAY_API bool IsOwnedBy(IXRSystemIdentifier* XRSystem) const;
	HEADMOUNTEDDISPLAY_API void Clear();
	
	bool IsSet() const 
	{ 
		// Since DeviceId's value is determined internally by the system, there's 
		// no agreed upon 'invalid' DeviceId, so this is the best way to detect a potentially bad identifier
		return !SystemName.IsNone(); 
	}

	HEADMOUNTEDDISPLAY_API bool operator==(const FXRDeviceId& Rhs) const;
	HEADMOUNTEDDISPLAY_API bool operator==(const IIdentifiableXRDevice* Rhs) const;

public:
	UPROPERTY(BlueprintReadOnly, Category="XRDevice")
	FName SystemName;
	UPROPERTY(BlueprintReadOnly, Category="XRDevice")
	int32 DeviceId;
};

