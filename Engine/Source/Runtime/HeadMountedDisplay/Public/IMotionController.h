// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "Features/IModularFeature.h"

#include "IMotionController.generated.h"

enum class EControllerHand : uint8;

UENUM(BlueprintType)
enum class ETrackingStatus : uint8
{
	NotTracked,
	InertialOnly, //Note This value of the enum is deprecated and only used by legacy platforms.  Please do not introduce new usages of it.
	Tracked,
};

/**
 * Motion Controller Source
 *
 * Named Motion Controller source. Used for UI display
 */
struct FMotionControllerSource
{
	FName SourceName;
#if WITH_EDITOR
	FName EditorCategory;
#endif

	FMotionControllerSource(FName InSourceName = NAME_None)
		: SourceName(InSourceName)
	{}
};

/**
 * Motion Controller device interface
 *
 * NOTE:  This intentionally does NOT derive from IInputDeviceModule, to allow a clean separation for devices which exclusively track motion with no tactile input
 * NOTE:  You must MANUALLY call IModularFeatures::Get().RegisterModularFeature( GetModularFeatureName(), this ) in your implementation!  This allows motion controllers
 *			to be both piggy-backed off HMD devices which support them, as well as standing alone.
 */

class IMotionController : public IModularFeature
{
public:
	virtual ~IMotionController() = default;

	static inline FName FeatureName = FName("MotionController");
	static FName GetModularFeatureName()
	{
		return FeatureName;
	}

	/**
	* Returns the device type of the controller.
	*
	* @return	Device type of the controller.
	*/
	virtual FName GetMotionControllerDeviceTypeName() const = 0;

	/**
	 * Returns the calibration-space orientation of the requested controller's hand.
	 *
	 * @param ControllerIndex	The Unreal controller (player) index of the controller set
	 * @param MotionSource		Which source, within the motion controller to get the orientation and position for
	 * @param OutOrientation	(out) If tracked, the orientation (in calibrated-space) of the controller in the specified hand
	 * @param OutPosition		(out) If tracked, the position (in calibrated-space) of the controller in the specified hand
	 * @param WorldToMetersScale The world scaling factor.
	 *
	 * @return					True if the device requested is valid and tracked, false otherwise
	 */
	virtual bool GetControllerOrientationAndPosition(const int32 ControllerIndex, const FName MotionSource, FRotator& OutOrientation, FVector& OutPosition, float WorldToMetersScale) const = 0;

	/**
	 * Returns the calibration-space orientation of the requested controller's hand.
	 *
	 * @param ControllerIndex	The Unreal controller (player) index of the controller set
	 * @param MotionSource		Which source, within the motion controller to get the orientation and position for
	 * @param OutOrientation	(out) If tracked, the orientation (in calibrated-space) of the controller in the specified hand
	 * @param OutPosition		(out) If tracked, the position (in calibrated-space) of the controller in the specified hand
	 * @param OutbProvidedLinearVelocity     (out) True if linear velocity was provided.
	 * @param OutLinearVelocity              (out) The Linear velocity of the controller.
	 * @param OutbProvidedAngularVelocity    (out) True if angular velocity was provided.
	 * @param OutAngularVelocityAsAxisAndLength    (out) The angular velocity of the controller represented as an axis of rotation who's magnitude is the velocity in radians per second.  See UHeadMountedDisplayFunctionLibrary::GetControllerTransformForTime2 for an example of how this can be turned into an FRotator without losing rotation speed beyond 180 degrees/second.
	 * @param OutbProvidedLinearAcceleration (out) True if linear acceleration was provided.
	 * @param OutLinearAcceleration          (out) The Linear acceleration of the controller.
	 * @param WorldToMetersScale The world scaling factor.
	 *
	 * @return					True if the device requested is valid and tracked, false otherwise
	 */
	virtual bool GetControllerOrientationAndPosition(const int32 ControllerIndex, const FName MotionSource, FRotator& OutOrientation, FVector& OutPosition, bool& OutbProvidedLinearVelocity, FVector& OutLinearVelocity, bool& OutbProvidedAngularVelocity, FVector& OutAngularVelocityAsAxisAndLength, bool& OutbProvidedLinearAcceleration, FVector& OutLinearAcceleration, float WorldToMetersScale) const = 0;

	/** Returns the calibration-space orientation of the requested controller's hand at the specified time for potentially improved temporal precision, particularly fetching the controller transform when a button was pressed
	* on a platform that provides sub-frame timing for button presses.  This is only intended to work with times very near the current frame.  In general it should be called immediatly after receiving the button press. 
	* On many platforms this functionality is not supported and this function will set OutTimeWasUsed to false and then call GetControllerOrientationAndPosition, ignoring Time.
	*
	* @param ControllerIndex                The Unreal controller (player) index of the controller set
	* @param MotionSource                   Which source, within the motion controller to get the orientation and position for
	* @param Time                           The time at which we would like to query the orientation.
	* @param OutTimeWasUsed                 (out) If true the time was used somehow to give a more temporally relevant orientation.  If false the time was ignored and a cached value returned.
	* @param OutOrientation                 (out) If tracked, the orientation (in calibrated-space) of the controller in the specified hand
	* @param OutPosition                    (out) If tracked, the position (in calibrated-space) of the controller in the specified hand
	* @param OutbProvidedLinearVelocity     (out) True if linear velocity was provided.
	* @param OutLinearVelocity              (out) The Linear velocity of the controller.
	* @param OutbProvidedAngularVelocity    (out) True if angular velocity was provided.
	* @param OutAngularVelocityAsAxisAndLength    (out) The angular velocity of the controller represented as an axis of rotation who's magnitude is the velocity in radians per second.  See UHeadMountedDisplayFunctionLibrary::GetControllerTransformForTime2 for an example of how this can be turned into an FRotator without losing rotation speed beyond 180 degrees/second.
	* @param OutbProvidedLinearAcceleration (out) True if linear acceleration was provided.
	* @param OutLinearAcceleration          (out) The Linear acceleration of the controller.
	* @param WorldToMetersScale             The world scaling factor.
	*
	* @return								True if the device requested is valid and tracked, false otherwise
	*/
	virtual bool GetControllerOrientationAndPositionForTime(const int32 ControllerIndex, const FName MotionSource, FTimespan Time, bool& OutTimeWasUsed, FRotator& OutOrientation, FVector& OutPosition, bool& OutbProvidedLinearVelocity, FVector& OutLinearVelocity, bool& OutbProvidedAngularVelocity, FVector& OutAngularVelocityAsAxisAndLength, bool& OutbProvidedLinearAcceleration, FVector& OutLinearAcceleration, float WorldToMetersScale) const = 0;

	static inline FRotator AngularVelocityAsAxisAndLengthToRotator(const FVector AngularVelocityAsAxisAndLength)
	{
		FVector Direction;
		double Angle;
		AngularVelocityAsAxisAndLength.ToDirectionAndLength(Direction, Angle);
		return FQuat(Direction, 1.0).Rotator() * Angle; // Note FQuat cannot represent >180 degree roration, so we build a quaternion of the correct axis for 1 radian and then scale the resulting rotator by the actual number of radians.
	}

	/**
	 * Returns the tracking status (e.g. not tracked, intertial-only, fully tracked) of the specified controller
	 *
	 * @return	Tracking status of the specified controller, or ETrackingStatus::NotTracked if the device is not found
	 */
	virtual ETrackingStatus GetControllerTrackingStatus(const int32 ControllerIndex, const FName MotionSource) const = 0;

	/**
	 * Called to request the motion sources that this IMotionController provides
	 *
	 * @param Sources	A motion source enumerator object that IMotionControllers can add source names to
	 */
	virtual void EnumerateSources(TArray<FMotionControllerSource>& SourcesOut) const = 0;

	/**
	 * Returns a custom names parameter value
	 *
	 * @param MotionSource		The name of the motion source we want parameters for
	 * @param ParameterName		The specific value we are looking for
	 * @param bOutValueFound	(out) Whether the parameter could be found
	 *
	 * @return			The value of the parameter
	 */
	virtual float GetCustomParameterValue(const FName MotionSource, FName ParameterName, bool& bOutValueFound) const = 0;
	
	virtual bool GetHandJointPosition(const FName MotionSource, int jointIndex, FVector& OutPosition) const = 0;

	/**
	* Add a set of enhanced input user settings to the motion controller. This allows the motion controller to support
	* Enhanced Input actions.
	*
	* @param InputConfig		The path to the enhanced user input settings asset
	*
	* @return			False if the input config can't be attached to the session, true otherwise
	*/
	virtual bool SetEnhancedInputUserSettings(TObjectPtr<class UEnhancedInputUserSettings> InputSettings = nullptr) { return true; };

	/**
	 * Add input mapping contexts to the motion controller. This allows the motion controller to support
	 * Enhanced Input actions.
	 *
	 * @param MappingContexts		The set of input mapping contexts used with motion controllers
	 *
	 * @return			False if the input mapping contexts can't be attached to the session, true otherwise
	 */
	virtual bool AttachInputMappingContexts(const TSet<TObjectPtr<class UInputMappingContext>>& InputMappingContexts) { return true; };

	// explicit source names
	static HEADMOUNTEDDISPLAY_API FName LeftHandSourceId;
	static HEADMOUNTEDDISPLAY_API FName RightHandSourceId;
	static HEADMOUNTEDDISPLAY_API FName HMDSourceId;
	static HEADMOUNTEDDISPLAY_API FName HeadSourceId; // Note Head and HMD are interchangable.  HMD is the legacy term.

	static HEADMOUNTEDDISPLAY_API bool GetHandEnumForSourceName(const FName Source, EControllerHand& OutHand);
};
