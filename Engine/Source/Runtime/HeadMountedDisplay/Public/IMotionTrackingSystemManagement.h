// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Features/IModularFeature.h"

enum class EControllerHand : uint8;

/**
 * Motion Tracking System Management interface
 *
 * This exposes management options for a motion tracking system.  Some platforms will not implement it.  
 */

class IMotionTrackingSystemManagement : public IModularFeature
{
public:
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName("MotionTrackingSystemManagement");
		return FeatureName;
	}

	/**
	* Set whether controller tracking is enabled by default or whether controllers must be specifically enabled.
	*
	* @param Enable						(in) True to enable by default.
	*/
	virtual void SetIsControllerMotionTrackingEnabledByDefault(bool Enable) = 0;

	/**
	* Get the maximum number of controllers that can be tracked.
	*
	* @return (int) number of controllers that can be tracked.
	*/
	virtual int32 GetMaximumMotionTrackedControllerCount() const = 0;

	/**
	* Get the number of controllers for which motion tracking is enabled.
	*
	* @return (int) number of controllers tracked now.
	*/
	virtual int32 GetMotionTrackingEnabledControllerCount() const = 0;

	/**
	* Returns true if the specified device is supposed to be tracked
	*
	* @param PlayerIndex				(in) The index of the player.
	* @param Hand						(in) The tracked device type.
	*
	* @return (Boolean) true if the specified device is set to be tracked.
	*/
	virtual bool IsMotionTrackingEnabledForDevice(const int32 ControllerIndex, const EControllerHand DeviceHand) const = 0;

	/**
	* Enaable tracking of the specified controller, by player index and tracked device type.
	*
	* @param PlayerIndex				(in) The index of the player.
	* @param Hand						(in) The device type.
	*
	* @return (Boolean) true if the specified device is now set to be tracked.  This could fail due to tracking limits, or on invalid input.
	*/
	virtual bool EnableMotionTrackingOfDevice(const int32 ControllerIndex, const EControllerHand DeviceHand) = 0;

	/**
	* Disable tracking of the specified controller, by player index and tracked device type.
	*
	* @param PlayerIndex				(in) The index of the player.
	* @param Hand						(in) The tracked device type.
	*/
	virtual void DisableMotionTrackingOfDevice(const int32 ControllerIndex, const EControllerHand DeviceHand) = 0;

	/**
	* Disable tracking for all controllers.
	*/
	virtual void DisableMotionTrackingOfAllControllers() = 0;

	/**
	* Disable tracking for all controllers of a certain player.
	*
	* @param PlayerIndex				(in) The index of the player.
	*/
	virtual void DisableMotionTrackingOfControllersForPlayer(int32 PlayerIndex) = 0;
};
