// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Features/IModularFeature.h"

enum class ESpectatorScreenMode : uint8;
class UTexture;
struct FSpectatorScreenModeTexturePlusEyeLayout;

/**
 * Spectator Screen Controller interface
 *
 * This is the interface to control the spectator screen, on platforms that support it.
 */

class ISpectatorScreenController : public IModularFeature
{
public:
	virtual ~ISpectatorScreenController() = default;

	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName("SpectatorScreenController");
		return FeatureName;
	}

	/**
	* Sets the social screen mode.
	* @param Mode				(in) The social screen Mode.
	*/
	virtual void SetSpectatorScreenMode(ESpectatorScreenMode Mode) = 0;

	virtual ESpectatorScreenMode GetSpectatorScreenMode() const = 0;

	/**
	* Change the texture displayed on the social screen
	* @param	InTexture: new Texture2D
	*/
	virtual void SetSpectatorScreenTexture(UTexture* InTexture) = 0;

	/**
	* Get the texture that would currently be displayed on the social screen (if in a mode that does that)
	*/
	virtual UTexture* GetSpectatorScreenTexture() const { return nullptr; }

	/**
	* Setup the layout for ESpectatorScreenMode::TexturePlusEye.
	* @param	EyeRectMin: min of screen rectangle the eye will be drawn in.  0-1 normalized.
	* @param	EyeRectMax: max of screen rectangle the eye will be drawn in.  0-1 normalized.
	* @param	TextureRectMin: min of screen rectangle the texture will be drawn in.  0-1 normalized.
	* @param	TextureRectMax: max of screen rectangle the texture will be drawn in.  0-1 normalized.
	* @param	bDrawEyeFirst: if true the eye is drawn before the texture, if false the reverse.
	* @param	bClearBlack: if true the render target will be drawn black before either rect is drawn.
	*/
	virtual void SetSpectatorScreenModeTexturePlusEyeLayout(const FSpectatorScreenModeTexturePlusEyeLayout& Layout) = 0;
};
