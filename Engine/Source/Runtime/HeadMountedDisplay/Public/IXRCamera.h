// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HeadMountedDisplayTypes.h"
#include "IIdentifiableXRDevice.h"
// @todo : needed because of EStereoscopicPass in clang; does not allow enum forward references.
//         We could remove this if EStereoscopicPass was a typed-enum.
#include "StereoRendering.h"

class APlayerController;
class FSceneInterface;
class USceneComponent;

/**
 * Interface used by the camera component to figure out the final position of a motion tracked camera.
 *
 * To reduce duplicated code, implementations should use the FXRCameraBase instead of implementing this interface directly.
 */
class  IXRCamera : public IIdentifiableXRDevice
{
public:
	IXRCamera() {}
	
	/**
	 * Set the view offset mode to assume an implied HMD position
	 */
	virtual void UseImplicitHMDPosition(bool bInImplicitHMDPosition) = 0;

	/**
	 * Gets the view offset mode to assume an implied HMD position
	 */
	virtual bool GetUseImplicitHMDPosition() const = 0;

	/**
	 * Optionally called by APlayerController to apply the orientation of the
	 * headset to the PC's rotation. If this is not done then the PC will face
	 * differently than the camera, which might be good (depending on the game).
	 */
	virtual void ApplyHMDRotation(APlayerController* PC, FRotator& ViewRotation) = 0;

	/**
	 * Apply the orientation and position of the headset to the Camera.
	 */
	virtual bool UpdatePlayerCamera(FQuat& CurrentOrientation, FVector& CurrentPosition, float DeltaTime) = 0;
	
	/**
	 * Override the Field of View for the player camera component.
	 */
	virtual void OverrideFOV(float& InOutFOV) = 0;

	/** Setup state for applying the render thread late update */
	virtual void SetupLateUpdate(const FTransform& ParentToWorld, USceneComponent* Component, bool bSkipLateUpdate) = 0;

	/**
	 * Calculates the offset for the camera position, given the specified eye pass, position and rotation.
	 * An XR plugin implementing stereo rendering should forward all calls of CalculateStereoViewOffset to this method.
	 */
	virtual void CalculateStereoCameraOffset(const int32 ViewIndex, FRotator& ViewRotation, FVector& ViewLocation) = 0;

	/**
	 * Fetches the UV coordinates of an AR passthrough camera relative to the screen. Takes into account cropping, orientation and difference between
	 * camera and screen aspect ratios. Returns false if the UVs are not available or if the feature is not supported or implemented by the current platform.
	 * If the result is true, OutUVs will contain the four corner UVs of the passthrough camera rectangle.
	 *
	 * This method should only be called from the render thread.
	 */
	virtual bool GetPassthroughCameraUVs_RenderThread(TArray<FVector2D>& OutUVs) { return false; }
};
