// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HeadMountedDisplayTypes.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Engine/Texture.h"


/**
 * Interface used to register and show loading splash screen layers.
 * A loading screen consists of one or more splash layers shown instead of the current scene during level load.
 *
 * To reduce duplicated code, implementations should use the FXRLoadingScreenBase of FDefaultXRLoadingScreen instead of implementing this interface directly.
 */
class  IXRLoadingScreen 
{
public:

	virtual ~IXRLoadingScreen() = default;

	/** 
	 * Structure describing the visual appearance of a single loading splash.
	 * 
	 * Splashes are shown in a tracker-relative space with the orientation reset to the direction
	 * the player is facing when brought up.
	 */
	struct FSplashDesc
	{
		FSplashDesc() = default;
		FSplashDesc(const FSplashDesc&) = default;
		FSplashDesc(FSplashDesc&&) = default;
		FSplashDesc& operator=(const FSplashDesc&) = default;
		FSplashDesc& operator=(FSplashDesc&&) = default;

		// Transform of the splash relative to the HMD orientation and location at the time of showing the loading screen.
		FTransform	Transform = FTransform::Identity;
		// Size of rendered quad in UE units
		FVector2D	QuadSize = FVector2D(1.0f, 1.0f);
		// UVs of the rendered texture
		FBox2D		UVRect = FBox2D(FVector2D(0.0f, 0.0f), FVector2D(1.0f, 1.0f));
		
		// Simple animation. Rotation that will be applied to the splash every other frame.
		FQuat		DeltaRotation = FQuat::Identity;

		// If set, the splash texture will be rendered opaque regardless of the alpha channel. Not supported by all platforms.
		bool		bIgnoreAlpha = false;

		// Set to true to tell that the texture is dynamically generated and may update each frame.
		bool		bIsDynamic	 = false;
		
		// Set this to true if the assigned texture is an external texture, for instance if using the media framework.
		bool		bIsExternal	 = false;

		// The texture shown. Can be set to a 2D or a Cube texture. (Cube textures may not be supported by all platforms.)
		// Implementations of IXRLoadingScreen will prevent GC of these pointers for any stored splashes.
		TWeakObjectPtr<UTexture>	TextureObj = nullptr;

		// If set, overrides the texture shown for the left eye. If null, both eyes will show the same texture.
		// Useful for stereo texture cubes.
		// Implementations of IXRLoadingScreen will prevent GC of these pointers for any stored splashes.
		TWeakObjectPtr<UTexture>	LeftTextureObj = nullptr;
	};

	/**
	 * Removes all splashes. Use this to replace the existing splashes before calling AddSplash.
	 */
	virtual void ClearSplashes() = 0;

	/** 
	 * Registers a splash to be shown while the loading screen is active.
	 *
	 * @param Splash	The settings for the new splash.
	 */
	virtual void AddSplash(const FSplashDesc& Splash) = 0;

	/** 
	 * Activates the loading screen.
	 * If called while the loading screen is active, this will reinitialize the positions of all splashes according to the current HMD pose.
	 */
	virtual void ShowLoadingScreen() = 0;

	/** 
	 * Hides the loading screen.
	 */
	virtual void HideLoadingScreen() = 0;

	/** 
	 * Returns whether the loading screen is currently active or not.
	 */
	virtual bool IsShown() const = 0;

	/**
	 * Returns whether a loading movie is playing or not.
	 */
	virtual bool IsPlayingLoadingMovie() const = 0;
};
