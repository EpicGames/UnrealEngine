// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Features/IModularFeature.h"
#include "IIdentifiableXRDevice.h" // for IXRSystemIdentifier

class UPrimitiveComponent;
class AActor;
enum class EControllerHand : uint8;

DECLARE_DELEGATE_OneParam(FXRComponentLoadComplete, UPrimitiveComponent*);

class IXRSystemAssets : public IModularFeature, public IXRSystemIdentifier
{
public:
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName("XRSystemAssets");
		return FeatureName;
	}

	/**
	 * Fills out a device list with unique identifiers that can be used
	 * to reference system specific devices. 
	 *
	 * These IDs are intended to be used with certain methods to reference a 
	 * specific device (like with CreateRenderComponent(), etc.).
	 * NOTE: that these IDs are NOT interoperable across XR systems (vive vs. oculus, 
	 * etc.). Using an ID from one system with another will have undefined results. 
	 *
	 * @param  DeviceListOut	A list of 
	 *
	 * @return True if the DeviceList was successfully updated, otherwise false. 
	 */
	virtual bool EnumerateRenderableDevices(TArray<int32>& DeviceListOut) = 0;

	/**
	 * Provides a mapping for MotionControllers, so we can identify the device ID
	 * used for a specific hand (which in turn can be used in other functions, 
	 * like CreateRenderComponent, etc.).
	 *
	 * NOTE: Some systems will not have an explicit mapping until the device is 
	 *       on and registered. In that case, this function will return an 
	 *       invalid device ID.
	 * 
	 * NOTE: Not all XR systems support every EControllerHand value. If that's
	 *       the case, then this will return an invalid device ID, though that
	 *       specific value is undetermined and up to each XRSystem for 
	 *       interpretation.
	 * 
	 * @param  ControllerHand	The UE specific hand identifier you want a system specific device ID for.
	 *
	 * @return A device ID that is specific to this XRSystem (can only be interpreted by other members belonging to the same IXRSystemIdentifier) 
	 */
	virtual int32 GetDeviceId(EControllerHand ControllerHand) = 0;

	/**
	 * Attempts to spawn a renderable component for the specified device. Returns
	 * a component that needs to be attached and registered by the caller.
	 *
	 * NOTE: Resource loads for this component may be asynchronous. The 
	 *       component can be attached and registered immediately, but there may 
	 *       be a delay before it renders properly. To make the operation 
	 *       synchronous use the bForceSynchronous param.
	 *
	 * @param  DeviceId				Uniquely identifies the XR device you want to render.
	 * @param  Owner				The actor which this component will be attached to.
	 * @param  Flags				Object creation flags to spawn the component with.
	 * @param  bForceSynchronous	Forces the any associated resource loads to complete before returning. 
	 * @param  OnLoadComplete		Model load callback, useful for when the load is asynchronous - should be triggered for synchronous loads as well.
	 *
	 * @return A valid component pointer if the method succeeded, otherwise null.
	 */
	virtual UPrimitiveComponent* CreateRenderComponent(const int32 DeviceId, AActor* Owner, EObjectFlags Flags, const bool bForceSynchronous = false, const FXRComponentLoadComplete& OnLoadComplete = FXRComponentLoadComplete()) = 0;
};
