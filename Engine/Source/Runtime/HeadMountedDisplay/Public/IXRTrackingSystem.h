// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HeadMountedDisplayTypes.h"
#include "IIdentifiableXRDevice.h"
#include "UObject/ObjectMacros.h"
#include "Features/IModularFeature.h"
#include "StereoRendering.h"

class IXRCamera;
class UARPin;
class FSceneViewFamily;
struct FWorldContext;
class FARSupportInterface;
class IOpenXRHMD;

/**
 * Struct representing the properties of an external tracking sensor.
 */
struct FXRSensorProperties
{
	/** The field of view of the sensor to the left in degrees */
	float LeftFOV;
	/** The field of view of the sensor to the right in degrees */
	float RightFOV;
	/** The upwards field of view of the sensor in degrees */
	float TopFOV;
	/** The downwards field of view of the sensor in degrees */
	float BottomFOV;
	/** The near plane of the sensor's effective tracking area */
	float NearPlane; 
	/** The far plane of the sensor's effective tracking area */
	float FarPlane;

	/** The focal distance of the camera. Can be zero if this does not make sense for the type of tracking sensor. */
	float CameraDistance;
};



/**
 * Main access point to an XR tracking system. Use it to enumerate devices and query their poses.
 */
class  IXRTrackingSystem : public IModularFeature, public IXRSystemIdentifier
{
public:
	static FName GetModularFeatureName()
	{
		static const FName FeatureName = FName("XRTrackingSystem");
		return FeatureName;
	}

	/**
	 * Returns version string.
	 */
	virtual FString GetVersionString() const = 0;

	/**
	 * Returns device specific flags.
	 */
	virtual int32 GetXRSystemFlags() const = 0;

	/**
	 * Device id 0 is reserved for an HMD. This should represent the HMD or the first HMD in case multiple HMDs are supported.
	 * Other devices can have arbitrary ids defined by each system.
	 * If a tracking system does not support tracking HMDs, device ID zero should be treated as invalid.
	 */
	static constexpr int32 HMDDeviceId = 0;

	/**
	 * Whether or not the system supports positional tracking (either via sensor or other means)
	 */
	virtual bool DoesSupportPositionalTracking() const = 0;

	/**
	 * Return true if the default camera implementation should query the current pose at the start of the render frame and apply late update.
	 * In order to support late update, the plugin should refresh the current pose just before rendering starts.
	 * A good point to insert the update is in OnBeginRendering_GameThread or OnBeginRendering_RenderThread.
	 *
	 * Note that for backwards compatibility with plugins written before 4.19, this method defaults to returning 'true'
	 */
	virtual bool DoesSupportLateUpdate() const { return true; }

	/**
	 * Return true if the default camera implementation should query the current projection matrix at the start of the render frame and apply late update.
	 * In order to support late update, the plugin should refresh the current projection matrix just before rendering starts.
	 * A good point to insert the update is in OnBeginRendering_GameThread or OnBeginRendering_RenderThread.
	 *
	 * Note that late projection update isn't compatible with all XR implementations because of projection matrix fetch restrictions.
	 */
	virtual bool DoesSupportLateProjectionUpdate() const { return false; }

	/**
	 * If the system currently has valid tracking positions. If not supported at all, returns false.
	 */
	virtual bool HasValidTrackingPosition() = 0;

	/**
	 * Reports all devices currently available to the system, optionally limiting the result to a given class of devices.
	 *
	 * @param OutDevices The device ids of available devices will be appended to this array.
	 * @param Type Optionally limit the list of devices to a certain type.
	 */
	virtual bool EnumerateTrackedDevices(TArray<int32>& OutDevices, EXRTrackedDeviceType Type = EXRTrackedDeviceType::Any) = 0;

	/**
	 * Get the count of tracked devices
	 * @param Type Optionally limit the count to a certain type
	 * @return the count of matching tracked devices
	 */
	virtual uint32 CountTrackedDevices(EXRTrackedDeviceType Type = EXRTrackedDeviceType::Any) = 0;

	/**
	 * Check current tracking status of a device.
	 * @param DeviceId the device to request status for.
	 * @return true if the system currently has valid tracking info for the given device ID.
	 */
	virtual bool IsTracking(int32 DeviceId) = 0;

	/** 
	 * Get the current pose for a device.
	 * This method must be callable both on the render thread and the game thread.
	 * For devices that don't support positional tracking, OutPosition will be at the base position.
	 *
	 * @param DeviceId the device to request the pose for.
	 * @param OutOrientation The current orientation of the device
	 * @param OutPosition The current position of the device
	 * @return true if the pose is valid or not.
	 */
	virtual bool GetCurrentPose(int32 DeviceId, FQuat& OutOrientation, FVector& OutPosition) = 0;

	/** 
	 * If the device id represents a head mounted display, fetches the relative position of the given eye relative to the eye.
	 * If the device is does not represent a stereoscopic tracked camera, orientation and position should be identity and zero and the return value should be false.
	 *
	 * @param DeviceId the device to request the eye pose for.
	 * @param ViewIndex the view the pose should be requested for, if passing in INDEX_NONE, the method should return a zero offset.
	 * @param OutOrientation The orientation of the eye relative to the device orientation.
	 * @param OutPosition The position of the eye relative to the tracked device
	 * @return true if the pose is valid or not. If the device is not a stereoscopic device, return false.
	 */
	virtual bool GetRelativeEyePose(int32 DeviceId, int32 ViewIndex, FQuat& OutOrientation, FVector& OutPosition) = 0;

	/** 
	 * If the device id represents a tracking sensor, reports the frustum properties in game-world space of the sensor.
	 * @param DeviceId the device to request information for.
	 * @param OutOrientation The current orientation of the device.
	 * @param OutPosition The current position of the device.
	 * @param OutSensorProperties A struct containing the tracking sensor properties.
	 * @return true if the device tracking is valid and supports returning tracking sensor properties.
	 */
	virtual bool GetTrackingSensorProperties(int32 DeviceId, FQuat& OutOrientation, FVector& OutPosition, FXRSensorProperties& OutSensorProperties) = 0;

	/**
	 * If the device id represents a tracking sensor, reports the device type.
	 * @param DeviceId the device to request information for.
	 * @return the device type enum.
	 */
	virtual EXRTrackedDeviceType GetTrackedDeviceType(int32 DeviceId) const = 0;

	/**
	 * If the device id represents a tracking sensor, reports the serial number as a string if the device supports it.
	 * @param DeviceId the device to request information for.
	 * @return the serial number of the device if it's available.
	 */
	virtual FString GetTrackedDevicePropertySerialNumber(int32 DeviceId) = 0;

	/**
	 * Sets tracking origin (either 'eye'-level or 'floor'-level).
	 */
	virtual void SetTrackingOrigin(EHMDTrackingOrigin::Type NewOrigin) = 0;

	/**
	 * Returns current tracking origin.
	 */
	virtual EHMDTrackingOrigin::Type GetTrackingOrigin() const = 0;

	/**
	 * Returns the system's latest known tracking-to-world transform.
	 * Useful for translating poses from GetCurrentPose() into unreal world space.
	 */
	virtual FTransform GetTrackingToWorldTransform() const = 0;

	/**
	 * This method should return the world to meters scale for the current frame.
	 * Should be callable on both the render and the game threads.
	 * @return the current world to meter scale.
	 */
	virtual float GetWorldToMetersScale() const = 0;

	/** 
	 * Computes a transform to convert from 'Floor' origin space to 'Eye' origin space.
	 * Useful when changing between the two different TrackingOrigin spaces.
	 * Invert the transform to get the opposite.
	 *
	 * @param  OutFloorToEye	[out] The returned floor-to-eye transform.
	 * @return True if the transform was successfully constructed.
	 */
	virtual bool GetFloorToEyeTrackingTransform(FTransform& OutFloorToEye) const = 0;

	/**
	 * Refreshes the system's known tracking-to-world transform.
	 * Helpful for clients if they change the world's representation of the XR origin, or if they want to override the system calculated 
	 * transform - calling this will update the known transform returned by GetTrackingToWorldTransform().
	 */
	virtual void UpdateTrackingToWorldTransform(const FTransform& TrackingToWorldOverride) = 0;

	/**
	 * Get the offset, in device space, of the reported device (screen / eye) position to the center of the head.
	 *
	 * @return a vector containing the offset coordinates, ZeroVector if not supported.
	 */
	virtual FVector GetAudioListenerOffset(int32 DeviceId = HMDDeviceId) const { return FVector::ZeroVector; }

	/**
	* Resets orientation by setting roll and pitch to 0, assuming that current yaw is forward direction and assuming
	* current position as a 'zero-point' (for positional tracking).
	*
	* @param Yaw				(in) the desired yaw to be set after orientation reset.
	*/
	virtual void ResetOrientationAndPosition(float Yaw = 0.f) = 0;

	/**
	* Resets orientation by setting roll and pitch to 0, assuming that current yaw is forward direction. Position is not changed.
	*
	* @param Yaw				(in) the desired yaw to be set after orientation reset.
	*/
	virtual void ResetOrientation(float Yaw = 0.f) {}

	/**
	* Resets position, assuming current position as a 'zero-point'.
	*/
	virtual void ResetPosition() {}

	/**
	* Sets base orientation by setting yaw, pitch, roll, assuming that this is forward direction.
	* Position is not changed.
	*
	* @param BaseRot			(in) the desired orientation to be treated as a base orientation.
	*/
	virtual void SetBaseRotation(const FRotator& BaseRot) {}

	/**
	* Returns current base orientation of HMD as yaw-pitch-roll combination.
	*/
	virtual FRotator GetBaseRotation() const { return FRotator::ZeroRotator; }

	/**
	* Sets base orientation, assuming that this is forward direction.
	* Position is not changed.
	*
	* @param BaseOrient		(in) the desired orientation to be treated as a base orientation.
	*/
	virtual void SetBaseOrientation(const FQuat& BaseOrient) {}

	/**
	* Returns current base orientation of HMD as a quaternion.
	*/
	virtual FQuat GetBaseOrientation() const { return FQuat::Identity; }

	/**
	* Sets base position of the HMD.
	*
	* @param BasePosition		(in) the desired offset to be treated as a base position.
	*/
	virtual void SetBasePosition(const FVector& BasePosition) {};

	/**
	* Returns current base position of HMD.
	*/
	virtual FVector GetBasePosition() const { return FVector::ZeroVector; }

	/**
	* Called to calibrate the offset transform between an external tracking source and the internal tracking source
	* (e.g. mocap tracker to and HMD tracker).  This should be called once per session, or when the physical relationship
	* between the external tracker and internal tracker changes (e.g. it was bumped or reattached).  After calibration,
	* calling UpdateExternalTrackingPosition will try to correct the internal tracker to the calibrated offset to prevent
	* drift between the two systems
	*
	* @param ExternalTrackingTransform		(in) The transform in world-coordinates, of the reference marker of the external tracking system
	*/
	virtual void CalibrateExternalTrackingSource(const FTransform& ExternalTrackingTransform) {}

	/**
	* Called after calibration to attempt to pull the internal tracker (e.g. HMD tracking) in line with the external tracker
	* (e.g. mocap tracker).  This will set the internal tracker's base offset and rotation to match and realign the two systems.
	* This can be called every tick, or whenever realignment is desired.  Note that this may cause choppy movement if the two
	* systems diverge relative to each other, or a big jump if called infrequently when there has been significant drift
	*
	* @param ExternalTrackingTransform		(in) The transform in world-coordinates, of the reference marker of the external tracking system
	*/
	virtual void UpdateExternalTrackingPosition(const FTransform& ExternalTrackingTransform) {}

	/** 
	 * Get the IXCamera instance for the given device.
	 *
	 * @param DeviceId the device the camera should track.
	 * @return a shared pointer to an IXRCamera.
	 */
	virtual class TSharedPtr< class IXRCamera, ESPMode::ThreadSafe > GetXRCamera(int32 DeviceId = HMDDeviceId) = 0;

	/** 
	 * Access HMD rendering-related features.
	 *
	 * @return a IHeadmountedDisplay pointer or a nullptr if this tracking system does not support head mounted displays.
	 */
	virtual class IHeadMountedDisplay* GetHMDDevice() { return nullptr; }

	/**
	* Access Stereo rendering device associated with this XR system.
	* If GetHMDDevice() returns non-null, this method should also return a vaild instance.
	*
	* @return a IStereoRendering pointer or a nullptr if this tracking system does not support stereo rendering.
	*/
	virtual class TSharedPtr< class IStereoRendering, ESPMode::ThreadSafe > GetStereoRenderingDevice()
	{
		check(GetHMDDevice() == nullptr);
		return nullptr; 
	}


	/**
	* Access optionsal ARCompositionComponent
	**/
	virtual TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> GetARCompositionComponent() { return nullptr; }
	virtual const TSharedPtr<const FARSupportInterface, ESPMode::ThreadSafe> GetARCompositionComponent() const { return nullptr; }


	/**
	 * Access the loading screen interface associated with this tracking system, if any.
	 *
	 * @return an IXRLoadingScreen pointer or a nullptr if this tracking system does not support loading screens.
	 */
	virtual class IXRLoadingScreen* GetLoadingScreen() { return nullptr; }

	/*** XR System related methods moved from IHeadMountedDisplay ***/

	/**
	* Returns true, if head tracking is allowed. Most common case: it returns true when GEngine->IsStereoscopic3D() is true,
	* but some overrides are possible.
	*/
	virtual bool IsHeadTrackingAllowed() const = 0;

	/**
	 * Same as IsHeadTrackingAllowed, but returns false if the World is not using VR (such as with the non-VR PIE instances when using VR Preview)
	 **/
	HEADMOUNTEDDISPLAY_API virtual bool IsHeadTrackingAllowedForWorld(UWorld & World) const;

	/** 
	* Can be used to enforce tracking even when stereo rendering is disabled. 
	* The default implementation does not allow enforcing tracking and always returns false.
	* This method is called both from the game and render threads.
	*/
	virtual bool IsHeadTrackingEnforced() const { return false; }

	/**
	* Can be used to enforce tracking even when stereo rendering is disabled.
	* The default implementation does not allow enforcing tracking and ignores the argument.
	*/
	virtual void SetHeadTrackingEnforced(bool bEnabled) {};


	/**
	* This method is called when playing begins. Useful to reset all runtime values stored in the plugin.
	*/
	virtual void OnBeginPlay(FWorldContext& InWorldContext) {}

	/**
	* This method is called when playing ends. Useful to reset all runtime values stored in the plugin.
	*/
	virtual void OnEndPlay(FWorldContext& InWorldContext) {}

	/**
	* This method is called when new game frame begins (called on a game thread).
	*/
	virtual bool OnStartGameFrame(FWorldContext& WorldContext) { return false; }

	/**
	* This method is called when game frame ends (called on a game thread).
	*/
	virtual bool OnEndGameFrame(FWorldContext& WorldContext) { return false; }


	/*** Methods designed to be called from IXRCamera implementations ***/

	/**
	 * Called just before rendering the current frame on the render thread. Invoked before applying late update, so plugins that want to refresh poses on the
	 * render thread prior to late update. Use this to perform any initializations prior to rendering.
	 */
	HEADMOUNTEDDISPLAY_API virtual void OnBeginRendering_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& ViewFamily);

	/**
	 * Called just before rendering the current frame on the game frame.
	 */
	virtual void OnBeginRendering_GameThread(FSceneViewFamily& InViewFamily) {}

	/**
	 * Called just after the late update on the render thread passing back the current relative transform.
	 */
	HEADMOUNTEDDISPLAY_API virtual void OnLateUpdateApplied_RenderThread(FRDGBuilder& GraphBuilder, const FTransform& NewRelativeTransform);

	/**
	 * Platform Agnostic Query about HMD details
	 */
	HEADMOUNTEDDISPLAY_API virtual void GetHMDData(UObject* WorldContext, FXRHMDData& HMDData);

	/**
	 * Platform Agnostic Query about MotionControllers details
	 */
	virtual void GetMotionControllerState(UObject* WorldContext, const EXRSpaceType XRSpaceType, const EControllerHand Hand, const EXRControllerPoseType XRControllerPoseType, FXRMotionControllerState& MotionControllerState) = 0;
	virtual void GetHandTrackingState(UObject* WorldContext, const EXRSpaceType XRSpaceType, const EControllerHand Hand, FXRHandTrackingState& HandTrackingState) = 0;
	virtual bool GetCurrentInteractionProfile(const EControllerHand Hand, FString& InteractionProfile) = 0;

	virtual EXRDeviceConnectionResult::Type ConnectRemoteXRDevice(const FString& IpAddress, const int32 BitRate)
	{ 
		return EXRDeviceConnectionResult::FeatureNotSupported;
	}
	virtual void DisconnectRemoteXRDevice() {}

	/**
	 * Get the bounds of the area where the user can freely move while remaining tracked centered around the specified origin
	 */
	virtual FVector2D GetPlayAreaBounds(EHMDTrackingOrigin::Type Origin) const { return FVector2D::ZeroVector; }

	/**
	 * Get the transform of the specified tracking origin, if available.
	 */
	virtual bool GetTrackingOriginTransform(TEnumAsByte<EHMDTrackingOrigin::Type> Origin, FTransform& OutTransform) const { return false; }

	/**
	 * Get the transform and dimensions of the area where the user can freely move while remaining tracked centered around the specified origin transform
	 */
	virtual bool GetPlayAreaRect(FTransform& OutTransform, FVector2D& OutRect) const { return false; }

	/**
	* Get the IOpenXRHMD interface, if there is one.
	*/
	virtual IOpenXRHMD* GetIOpenXRHMD() { return nullptr; }
};
