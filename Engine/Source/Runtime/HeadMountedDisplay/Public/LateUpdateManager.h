// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

class USceneComponent;
class FSceneInterface;
class FPrimitiveSceneInfo;

/**
* Utility class for applying an offset to a hierarchy of components in the renderer thread.
*/
class FLateUpdateManager
{
public:
	FLateUpdateManager() {}
	virtual ~FLateUpdateManager() = default;

	/** Setup state for applying the render thread late update */
	HEADMOUNTEDDISPLAY_API void Setup(const FTransform& ParentToWorld, USceneComponent* Component, bool bSkipLateUpdate);

	/** Returns true if the LateUpdateSetup data is stale. */
	bool GetSkipLateUpdate_RenderThread() const { return PipelinedUpdateStatesRendering.bSkip; }

	/** Apply the late update delta to the cached components */
	HEADMOUNTEDDISPLAY_API void Apply_RenderThread(FSceneInterface* Scene, const FTransform& OldRelativeTransform, const FTransform& NewRelativeTransform);

private:

	/** A utility method that calls CacheSceneInfo on ParentComponent and all of its descendants */
	HEADMOUNTEDDISPLAY_API void GatherLateUpdatePrimitives(USceneComponent* ParentComponent);
	/** Generates a LateUpdatePrimitiveInfo for the given component if it has a SceneProxy and appends it to the current LateUpdatePrimitives array */
	HEADMOUNTEDDISPLAY_API void CacheSceneInfo(USceneComponent* Component);

	struct FLateUpdateState
	{
		FLateUpdateState()
			: ParentToWorld(FTransform::Identity)
			, bSkip(false)
		{}

		/** Parent world transform used to reconstruct new world transforms for late update scene proxies */
		FTransform ParentToWorld;
		/** Primitives that need late update before rendering */
		TMap<FPrimitiveSceneInfo*, int32> Primitives;
		/** Late Update Info Stale, if this is found true do not late update */
		bool bSkip;
	};

	FLateUpdateState PipelinedUpdateStatesGame;
	FLateUpdateState PipelinedUpdateStatesRendering;
};

