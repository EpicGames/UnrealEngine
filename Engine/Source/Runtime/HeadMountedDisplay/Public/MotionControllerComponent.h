// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PrimitiveComponent.h"
#include "SceneViewExtension.h"
#include "IMotionController.h"
#include "LateUpdateManager.h"
#include "MotionControllerComponent.generated.h"

class FPrimitiveSceneInfo;
class FRHICommandListImmediate;
class FSceneView;
class FSceneViewFamily;
class UXRDeviceVisualizationComponent;

UCLASS(Blueprintable, meta = (BlueprintSpawnableComponent), ClassGroup = MotionController, MinimalAPI)
class UMotionControllerComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

	/** Which player index this motion controller should automatically follow */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetAssociatedPlayerIndex, Category = "MotionController")
	int32 PlayerIndex;

	/** Defines which pose this component should receive from the OpenXR Runtime. Left/Right MotionSource is the same as LeftGrip/RightGrip. See OpenXR specification for details on poses. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetTrackingMotionSource, Category = "MotionController")
	FName MotionSource;

	/** If false, render transforms within the motion controller hierarchy will be updated a second time immediately before rendering. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MotionController")
	uint32 bDisableLowLatencyUpdate : 1;

	/** The tracking status for the device (e.g. full tracking, inertial tracking only, no tracking) */
	UPROPERTY(BlueprintReadOnly, Category = "MotionController")
	ETrackingStatus CurrentTrackingStatus;

	/** Whether or not this component had a valid tracked device this frame */
	UFUNCTION(BlueprintPure, Category = "MotionController")
	bool IsTracked() const
	{
		return bTracked;
	}

	UFUNCTION(BlueprintSetter, meta = (DeprecatedFunction, DeprecationMessage = "Please use the Motion Source property instead of Hand"))
	HEADMOUNTEDDISPLAY_API void SetTrackingSource(const EControllerHand NewSource);

	UFUNCTION(BlueprintGetter, meta = (DeprecatedFunction, DeprecationMessage = "Please use the Motion Source property instead of Hand"))
	HEADMOUNTEDDISPLAY_API EControllerHand GetTrackingSource() const;

	UFUNCTION(BlueprintSetter)
	HEADMOUNTEDDISPLAY_API void SetTrackingMotionSource(const FName NewSource);

	HEADMOUNTEDDISPLAY_API FName GetTrackingMotionSource();

	UFUNCTION(BlueprintSetter)
	HEADMOUNTEDDISPLAY_API void SetAssociatedPlayerIndex(const int32 NewPlayer);

	HEADMOUNTEDDISPLAY_API void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	HEADMOUNTEDDISPLAY_API void BeginDestroy() override;

public:
	//~ UObject interface
	HEADMOUNTEDDISPLAY_API virtual void Serialize(FArchive& Ar) override;

	//~ UActorComponent interface
	HEADMOUNTEDDISPLAY_API virtual void InitializeComponent() override;

	// Delegate for activation of XRDeviceVisualizationComponent
	DECLARE_MULTICAST_DELEGATE_OneParam(FActivateVisualizationComponent, bool);
	static HEADMOUNTEDDISPLAY_API FActivateVisualizationComponent OnActivateVisualizationComponent;

protected:
	//~ Begin UActorComponent Interface.
	HEADMOUNTEDDISPLAY_API virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	HEADMOUNTEDDISPLAY_API virtual void SendRenderTransform_Concurrent() override;
	//~ End UActorComponent Interface.

	// Cached Motion Controller that can be read by GetParameterValue. Only valid for the duration of OnMotionControllerUpdated
	IMotionController* InUseMotionController;

	/** Blueprint Implementable function for responding to updated data from a motion controller (so we can use custom parameter values from it) */
	UFUNCTION(BlueprintImplementableEvent, Category = "Motion Controller Update")
	HEADMOUNTEDDISPLAY_API void OnMotionControllerUpdated();

	// Returns the value of a custom parameter on the current in use Motion Controller (see member InUseMotionController). Only valid for the duration of OnMotionControllerUpdated 
	UFUNCTION(BlueprintCallable, Category = "Motion Controller Update")
	HEADMOUNTEDDISPLAY_API float GetParameterValue(FName InName, bool& bValueFound);

	UFUNCTION(BlueprintCallable, Category = "Motion Controller Update")
	HEADMOUNTEDDISPLAY_API FVector GetHandJointPosition(int jointIndex, bool& bValueFound);

	/** If the motion tracking system provides linear velocity at this time the vector will be that velocity in cm/s in unreal world space and the function will return true.  If velocity is unavailable it will return false.  */
	UFUNCTION(BlueprintPure, Category = "Motion Controller Update")
	HEADMOUNTEDDISPLAY_API bool GetLinearVelocity(FVector &OutLinearVelocity) const;

	/** If the motion tracking system provides angular velocity at this time OutAngularVelocity will be that velocity in deg/s in unreal world space and the function will return true. Note that it is not difficult to rotate a controller at more than 0.5 or 1 rotation per second briefly and some mathmatical operations(such as conversion to quaternion) lose rotations beyond 180 degrees or 360 degrees..  In some cases that is OK becuase the resulting final rotation is the same, but in some cases it would generate incorrect results.   If angular velocity is unavailable it will return false.*/
	UFUNCTION(BlueprintPure, Category = "Motion Controller Update")
	HEADMOUNTEDDISPLAY_API bool GetAngularVelocity(FRotator& OutAngularVelocity) const;

	/** If the motion tracking system provides linear acceleration at this time the vector will be that acceleration in cm/(s^2) in unreal world space and the function will return true.  If acceleration is unavailable it will return false.  */
	UFUNCTION(BlueprintPure, Category = "Motion Controller Update")
	HEADMOUNTEDDISPLAY_API bool GetLinearAcceleration(FVector& OutLinearAcceleration) const;


protected:

	/** Whether or not this component had a valid tracked controller associated with it this frame*/
	bool bTracked;

	/** Whether or not this component has authority within the frame*/
	bool bHasAuthority;

	// Velocity and acceleration data for the motion controller, if available.
	bool bProvidedLinearVelocity;
	bool bProvidedAngularVelocity;
	bool bProvidedLinearAcceleration;
	// Note: these values are in tracking space, which is also the space relative to the parent of the motioncontroller component, and still need to be converted to unreal space by the TrackingToWorldTransform for use.
	// They are in the unreal coordinate system and world scaled.
	// Also we do not late-update these values.
	FVector LinearVelocity;
	FVector AngularVelocityAsAxisAndLength; // This vector represents an axis of rotation and its length is the magnitude of the rotation in radians per second.  Be careful turning it into a quaternion because those cannot represent more than 0.5 revolution per second. See UHeadMountedDisplayFunctionLibrary::GetControllerTransformForTime2 for an example of how this can be turned into an FRotator without losing rotation speed beyond 180 degrees/second.
	FVector LinearAcceleration;

	/** If true, the Position and Orientation args will contain the most recent controller state */
	HEADMOUNTEDDISPLAY_API bool PollControllerState_GameThread(FVector& Position, FRotator& Orientation, bool& OutbProvidedLinearVelocity, FVector& OutLinearVelocity, bool& OutbProvidedAngularVelocity, FVector& OutAngularVelocityAsAxisAndLength, bool& OutbProvidedLinearAcceleration, FVector& OutLinearAcceleration, float WorldToMetersScale);
	HEADMOUNTEDDISPLAY_API bool PollControllerState_RenderThread(FVector& Position, FRotator& Orientation, float WorldToMetersScale);

	HEADMOUNTEDDISPLAY_API void OnModularFeatureUnregistered(const FName& Type, class IModularFeature* ModularFeature);
	IMotionController* PolledMotionController_GameThread;
	IMotionController* PolledMotionController_RenderThread;
	bool bPolledHMD_GameThread;
	bool bPolledHMD_RenderThread;
	FCriticalSection PolledMotionControllerMutex; // Used to protect PolledMotionController_GameThread and bPolledHMD_GameThread which are written on the game thread and copied on the render thread.


	FTransform RenderThreadRelativeTransform;
	FVector RenderThreadComponentScale;

	/** View extension object that can persist on the render thread without the motion controller component */
	class FViewExtension : public FSceneViewExtensionBase
	{
	public:
		FViewExtension(const FAutoRegister& AutoRegister, UMotionControllerComponent* InMotionControllerComponent);
		virtual ~FViewExtension() = default;

		/** ISceneViewExtension interface */
		virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) override;
		virtual void PreRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily) override;
		virtual int32 GetPriority() const override { return -10; }
		virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const;

	protected:
		friend class UMotionControllerComponent;

		/** Motion controller component associated with this view extension */
		UMotionControllerComponent* MotionControllerComponent;
		FLateUpdateManager LateUpdate;
	};
	TSharedPtr< FViewExtension, ESPMode::ThreadSafe > ViewExtension;	
};
