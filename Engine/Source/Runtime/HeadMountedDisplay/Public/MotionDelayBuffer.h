// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "LateUpdateManager.h"
#include "UObject/WeakObjectPtr.h"

#include "SceneViewExtension.h"

class USceneComponent;
class FSceneInterface;
class FMotionDelayBuffer;
class FSceneViewFamily;
class FMotionDelayTarget;
class FMotionDelayClient;


/* FMotionDelayService
 *****************************************************************************/

class FMotionDelayService
{
public:
	static HEADMOUNTEDDISPLAY_API void SetEnabled(bool bEnable);
	static HEADMOUNTEDDISPLAY_API bool RegisterDelayTarget(USceneComponent* MotionControlledComponent, const int32 PlayerIndex, const FName SourceId);
	static HEADMOUNTEDDISPLAY_API void RegisterDelayClient(TSharedRef<FMotionDelayClient, ESPMode::ThreadSafe> DelayClient);
};

/* FMotionDelayClient
 *****************************************************************************/

class FMotionDelayClient : public FSceneViewExtensionBase
{
public:
	HEADMOUNTEDDISPLAY_API FMotionDelayClient(const FAutoRegister& AutoRegister);

	virtual uint32 GetDesiredDelay() const = 0;
	virtual void GetExemptTargets(TArray<USceneComponent*>& ExemptTargets) const {}

	HEADMOUNTEDDISPLAY_API void Apply_RenderThread(FSceneInterface* Scene);
	HEADMOUNTEDDISPLAY_API void Restore_RenderThread(FSceneInterface* Scene);

public:
	/** ISceneViewExtension interface */
	HEADMOUNTEDDISPLAY_API virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) override;
	HEADMOUNTEDDISPLAY_API virtual void PostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily) override;
	HEADMOUNTEDDISPLAY_API virtual int32 GetPriority() const override;
	HEADMOUNTEDDISPLAY_API virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override;

protected:
	HEADMOUNTEDDISPLAY_API bool FindDelayTransform(USceneComponent* Target, uint32 Delay, FTransform& TransformOut);

private:
	struct FTargetTransform
	{
		TSharedPtr<FMotionDelayTarget, ESPMode::ThreadSafe> DelayTarget;
		FTransform DelayTransform;
		FTransform RestoreTransform;
	};
	TArray<FTargetTransform> TargetTransforms_RenderThread;
};


/* TCircularHistoryBuffer
 *****************************************************************************/

/**
 * Modeled after TCircularBuffer/Queue, but resizable with it's own stack-style
 * way of indexing (0 = most recent value added)
 */
template<typename ElementType> class TCircularHistoryBuffer
{
public:
	TCircularHistoryBuffer(uint32 InitialCapacity = 0);

	ElementType& Add(const ElementType& Element);
	void Resize(uint32 NewCapacity);

	/**
	 * NOTE: Will clamp to the oldest value available if the buffer isn't full
	 *       and the index is larger than the number of values buffered.
	 *
	 * @param  Index	Stack-esque indexing: 0 => the most recent value added, 1+n => Older entries, 1+n back from the newest.
	 */
	ElementType& operator[](uint32 Index);
	const ElementType& operator[](uint32 Index) const;

	void InsertAt(uint32 Index, const ElementType& Element);

	uint32 Capacity() const;
	int32  Num() const;

	void Empty();

	bool IsEmpty() const;
	bool IsFull() const;
	
private:
	void RangeCheck(const uint32 Index, bool bCheckIfUnderfilled = false) const;
	uint32 AsInternalIndex(uint32 Index) const;
	uint32 GetNextIndex(uint32 CurrentIndex) const;

	void ResizeGrow(uint32 AddedSlack);
	void Realign();
	void ResizeShrink(uint32 ShrinkAmount);

private:
	/** Holds the buffer's elements. */
	TArray<ElementType> Elements;
	/** Index to the next slot to be filled. NOTE: The last element added is at Head-1. */
	uint32 Head;
	/** Allows us to recognize the difference between an empty buffer and one where Head has looped back to the start. */
	bool bIsFull;
};

// Templatized implementation
#include "MotionDelayBuffer.inl" // IWYU pragma: export
