// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

template<typename ElementType>
TCircularHistoryBuffer<ElementType>::TCircularHistoryBuffer(uint32 InitialCapacity)
	: Head(0)
	, bIsFull(false)
{
	if (InitialCapacity > 0)
	{
		Resize(InitialCapacity);
	}
}

template<typename ElementType>
ElementType& TCircularHistoryBuffer<ElementType>::Add(const ElementType& Element)
{
	Elements[Head] = Element;
	ElementType& EntryRef = Elements[Head];

	Head = GetNextIndex(Head);
	bIsFull |= (Head == 0);

	return EntryRef;
}

template<typename ElementType>
void TCircularHistoryBuffer<ElementType>::Resize(uint32 NewCapacity)
{
	checkSlow(NewCapacity > 0);

	const uint32 CurrentSize = Elements.Num();
	if (NewCapacity > CurrentSize)
	{
		ResizeGrow(NewCapacity - CurrentSize);
	}
	else if (NewCapacity < CurrentSize)
	{
		ResizeShrink(CurrentSize - NewCapacity);
	}
}

template<typename ElementType>
inline ElementType& TCircularHistoryBuffer<ElementType>::operator[](uint32 Index)
{
	RangeCheck(Index);
	return Elements[AsInternalIndex(Index)];
}

template<typename ElementType>
inline const ElementType& TCircularHistoryBuffer<ElementType>::operator[](uint32 Index) const
{
	RangeCheck(Index);
	return Elements[AsInternalIndex(Index)];
}

template<typename ElementType>
void TCircularHistoryBuffer<ElementType>::InsertAt(uint32 Index, const ElementType& Element)
{
	RangeCheck(Index);

	Index = FMath::Min(Index, (uint32)Num());
	if (!IsFull() || Index < Capacity() - 1)
	{
		uint32 NumToShift  = FMath::Min(Head, Index + 1);
		uint32 StartIndex  = Head - NumToShift;
		uint32 TargetIndex = AsInternalIndex(Index);

		Add(Element);

		uint32 ShiftCount = 0;
		if (NumToShift > 0)
		{
			FMemory::Memmove(&Elements[StartIndex + 1], &Elements[StartIndex], sizeof(ElementType) * NumToShift);
			ShiftCount += NumToShift;
		}

		if (ShiftCount < Index + 1)
		{
			Elements[0] = Elements.Last();
			ShiftCount += 1;
		}

		if (ShiftCount < Index + 1)
		{
			NumToShift = Elements.Num() - TargetIndex - 1;

			FMemory::Memmove(&Elements[TargetIndex + 1], &Elements[TargetIndex], sizeof(ElementType) * NumToShift);
			ShiftCount += NumToShift;
		}
		Elements[TargetIndex] = Element;
	}
}

template<typename ElementType>
inline uint32 TCircularHistoryBuffer<ElementType>::Capacity() const
{
	return Elements.Num();
}

template<typename ElementType>
inline int32 TCircularHistoryBuffer<ElementType>::Num() const
{
	if (IsFull())
	{
		return Capacity();
	}
	else
	{
		return Head;
	}
}

template<typename ElementType>
inline void TCircularHistoryBuffer<ElementType>::Empty()
{
	Head = 0;
	bIsFull = false;
}

template<typename ElementType>
inline bool TCircularHistoryBuffer<ElementType>::IsEmpty() const
{
	return (Head == 0) && !bIsFull;
}

template<typename ElementType>
inline bool TCircularHistoryBuffer<ElementType>::IsFull() const
{
	return bIsFull;
}

template<typename ElementType>
inline void TCircularHistoryBuffer<ElementType>::RangeCheck(const uint32 Index, bool bCheckIfUnderfilled) const
{
	checkSlow(!IsEmpty());
	checkSlow(Index < (uint32)Elements.Num());
	checkSlow(!bCheckIfUnderfilled || (!bIsFull && Index >= Head));
}

template<typename ElementType>
inline uint32 TCircularHistoryBuffer<ElementType>::AsInternalIndex(uint32 Index) const
{
	//        head(3)                                            head(3)
	//           |                                                  |
	// [2][1][0][6][5][4][3],  or (if not full):  [6/5/4/3/2][1][0][ ][ ][ ][ ]
	return (Index < Head) ? (Head - Index - 1) : bIsFull ? (Elements.Num() - 1 - Index + Head) : 0;
}

template<typename ElementType>
inline uint32 TCircularHistoryBuffer<ElementType>::GetNextIndex(uint32 CurrentIndex) const
{
	return ((CurrentIndex + 1) % Elements.Num());
}

template<typename ElementType>
void TCircularHistoryBuffer<ElementType>::ResizeGrow(uint32 AddedSlack)
{
	if (bIsFull)
	{
		if (Head > 0)
		{
			Realign();
		}
		Head = Elements.Num();
	}

	Elements.AddUninitialized(AddedSlack);
	bIsFull = false;
}

template<typename ElementType>
void TCircularHistoryBuffer<ElementType>::Realign()
{
	if (bIsFull)
	{
		const uint32 Capacity = Elements.Num();

		TArray<ElementType> TempBuffer;
		TempBuffer.AddUninitialized(Head);

		ElementType* BufferPtr = Elements.GetData();
		//        head(3)
		//           |                    
		// [G][H][I][C][D][E][F] => TempBuffer: [G][H][I]
		FMemory::Memcpy(TempBuffer.GetData(), BufferPtr, sizeof(ElementType) * Head);

		// [G][H][I][C][D][E][F] => [C][D][E][F][D][E][F]
		const uint32 MoveCount = Capacity - Head;
		FMemory::Memcpy(BufferPtr, BufferPtr + Head, sizeof(ElementType) * Head);

		// [C][D][E][F][D][E][F] => [C][D][E][F][G][H][I]
		FMemory::Memcpy(BufferPtr + MoveCount, TempBuffer.GetData(), sizeof(ElementType) * TempBuffer.Num());

		Head = 0;
	}
}

template<typename ElementType>
void TCircularHistoryBuffer<ElementType>::ResizeShrink(uint32 ShrinkAmount)
{
	const uint32 NewCapacity = Elements.Num() - ShrinkAmount;
	//
	// keep the newest values

	//        head(3)                  head(3)
	//           |                        |
	// [H][I][J][D][E][F][G] => [H][I][J][F][G]
	if (Head < NewCapacity)
	{
		Elements.RemoveAtSwap(Head, ShrinkAmount);
	}
	//        head(3)         head(0)
	//           |               |
	// [H][I][J][D][E][F][G] => [I][J]
	else
	{
		// [H][I][J][D][E][F][G] => [H][I][J]
		Elements.RemoveAt(Head, Elements.Num() - Head);
		// [H][I][J] => [I][J]
		Elements.RemoveAtSwap(0, Head - NewCapacity);

		Head = 0;
		bIsFull = true;
	}
}
